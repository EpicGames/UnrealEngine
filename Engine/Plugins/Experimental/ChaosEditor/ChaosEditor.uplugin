{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosEditor",
	"Description": "Destruction Tools",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "FractureEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PlanarCut",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "GeometryCollectionPlugin",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "Fracture",
			"Enabled": true
		}
	]
}