// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class FractureEditor : ModuleRules
{
	public FractureEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "Voronoi",
				"InteractiveToolsFramework",
				"EditorInteractiveToolsFramework"

				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"EditorScriptingUtilities",
				"ToolMenus",
				"UnrealEd",
				"LevelEditor",
                "GeometryCollectionEngine",
                "GeometryCollectionEditor",
				"ModelingComponents",
				"ModelingOperators",
				"PropertyEditor",
				"GeometryCore",
				"DataflowEngine",
				"DataflowEditor",
				"DynamicMesh",
				"MeshDescription",
				"StaticMeshDescription",
				"PlanarCut",
				"Chaos",
				"ToolWidgets",
				"DeveloperSettings",
				"FractureEngine",
				"ContentBrowserData",
				"GeometryFramework",
				"DataflowCore",
				"PhysicsCore"

				// ... add private dependencies that you statically link with here ...	
			}
            );
	}
}
