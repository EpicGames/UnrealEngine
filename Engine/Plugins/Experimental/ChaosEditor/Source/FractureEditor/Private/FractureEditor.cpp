// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEditor.h"
#include "FractureEditorMode.h"

#include "FractureEditorStyle.h"
#include "FractureEditorCommands.h"

#define LOCTEXT_NAMESPACE "FFractureEditorModule"

void FFractureEditorModule::StartupModule()
{
	FFractureEditorStyle::Get();

	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module


	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	FFractureEditorCommands::Register();

}

void FFractureEditorModule::ShutdownModule()
{
	// This function may be called during shutown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	
	FFractureEditorCommands::Unregister();
}


TSharedPtr<FExtensibilityManager> FFractureEditorModule::GetToolBarExtensibilityManager() const
{
	return ToolBarExtensibilityManager;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FFractureEditorModule, FractureEditor)