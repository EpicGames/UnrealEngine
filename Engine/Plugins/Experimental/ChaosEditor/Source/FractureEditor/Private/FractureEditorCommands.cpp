// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEditorCommands.h"

#include "FractureEditorStyle.h"
#include "FractureTool.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "FractureEditorCommands"

FFractureEditorCommands::FFractureEditorCommands()
	: TCommands<FFractureEditorCommands>("FractureEditor", LOCTEXT("Fracture", "Fracture"), NAME_None, FFractureEditorStyle::StyleName)
{
}

TSharedPtr<FUICommandInfo> FFractureEditorCommands::FindToolByName(const FString& Name, bool& bFound) const
{
	bFound = false;
	for (const FToolCommandInfo& Command : RegisteredTools)
	{
		if (Command.ToolUIName.ToString().Equals(Name, ESearchCase::IgnoreCase)
		 || FTextInspector::GetSourceString(Command.ToolUIName)->Equals(Name, ESearchCase::IgnoreCase))
		{
			bFound = true;
			return Command.ToolCommand;
		}
	}
	return TSharedPtr<FUICommandInfo>();
}

void FFractureEditorCommands::RegisterCommands()
{

	auto RegisterCommandInfo = [this](const TSharedPtr<FUICommandInfo>& CommandInfo) -> void
	{
		RegisteredTools.Add(FToolCommandInfo{ CommandInfo->GetLabel(), CommandInfo });
	};

	// View settings
	UI_COMMAND(ToggleShowBoneColors, "Colors", "Toggle Show Bone Colors", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::B) );
	RegisterCommandInfo(ToggleShowBoneColors);
	UI_COMMAND(ViewUpOneLevel, "ViewUpOneLevel", "View Up One Level", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::W) );
	RegisterCommandInfo(ViewUpOneLevel);
	UI_COMMAND(ViewDownOneLevel, "ViewDownOneLevel", "View Down One Level", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::S) );
	RegisterCommandInfo(ViewDownOneLevel);
	UI_COMMAND(ExplodeMore, "ExplodeMore", "Explode 10% More", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::E) );
	RegisterCommandInfo(ExplodeMore);
	UI_COMMAND(ExplodeLess, "ExplodeLess", "Explode 10% Less", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Q) );
	RegisterCommandInfo(ExplodeLess);
	UI_COMMAND(CancelTool, "CancelTool", "Exit Current Tool Mode", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	RegisterCommandInfo(CancelTool);

	// Fracture Tools
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UFractureActionTool::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			TSubclassOf<UFractureActionTool> SubclassOf = (*ClassIterator);
			UFractureActionTool* FractureTool = SubclassOf->GetDefaultObject<UFractureActionTool>();
			FractureTool->RegisterUICommand(this);
			RegisterCommandInfo(FractureTool->GetUICommandInfo());
		}
	}

}

#undef LOCTEXT_NAMESPACE
