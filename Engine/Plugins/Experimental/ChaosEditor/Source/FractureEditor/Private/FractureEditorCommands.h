// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"

class FFractureEditorCommands : public TCommands<FFractureEditorCommands>
{
	public:
		FFractureEditorCommands(); 

		virtual void RegisterCommands() override;

	protected:
		struct FToolCommandInfo
		{
			FText ToolUIName;
			TSharedPtr<FUICommandInfo> ToolCommand;
		};
		TArray<FToolCommandInfo> RegisteredTools;		// Tool commands listed below are stored in this list

	public:
		/**
		 * Find Tool start-command below by registered name (tool icon name in Mode palette)
		 */
		TSharedPtr<FUICommandInfo> FindToolByName(const FString& Name, bool& bFound) const;
		
		// Selection Commands
		TSharedPtr< FUICommandInfo > SelectAll;
		TSharedPtr< FUICommandInfo > SelectNone;
		TSharedPtr< FUICommandInfo > SelectNeighbors;
		TSharedPtr< FUICommandInfo > SelectParent;
		TSharedPtr< FUICommandInfo > SelectChildren;
		TSharedPtr< FUICommandInfo > SelectSiblings;
		TSharedPtr< FUICommandInfo > SelectAllInLevel;
		TSharedPtr< FUICommandInfo > SelectInvert;
		TSharedPtr< FUICommandInfo > SelectLeaves;
		TSharedPtr< FUICommandInfo > SelectClusters;
		TSharedPtr< FUICommandInfo > SelectCustom;

		// View Settings
		TSharedPtr< FUICommandInfo > ToggleShowBoneColors;
		TSharedPtr< FUICommandInfo > ViewUpOneLevel;
		TSharedPtr< FUICommandInfo > ViewDownOneLevel;
		TSharedPtr< FUICommandInfo > ExplodeMore;
		TSharedPtr< FUICommandInfo > ExplodeLess;

		// Tool exit
		TSharedPtr< FUICommandInfo > CancelTool;

		// Cluster Commands
		TSharedPtr< FUICommandInfo > AutoCluster;
		TSharedPtr< FUICommandInfo > ClusterMagnet;
		TSharedPtr< FUICommandInfo > Cluster;
		TSharedPtr< FUICommandInfo > Uncluster;
		TSharedPtr< FUICommandInfo > Flatten;
		TSharedPtr< FUICommandInfo > MoveUp;
		TSharedPtr< FUICommandInfo > ClusterMerge;

		// Edit Commands
		TSharedPtr< FUICommandInfo > DeleteBranch;
		TSharedPtr< FUICommandInfo > Hide;
		TSharedPtr< FUICommandInfo > Unhide;
		TSharedPtr< FUICommandInfo > MergeSelected;
		TSharedPtr< FUICommandInfo > SplitSelected;
		
		// Generate Commands
		TSharedPtr< FUICommandInfo > GenerateAsset;
		TSharedPtr< FUICommandInfo > ResetAsset;

		// Embed Commands
		TSharedPtr< FUICommandInfo > AddEmbeddedGeometry;
		TSharedPtr< FUICommandInfo > AutoEmbedGeometry;
		TSharedPtr< FUICommandInfo > FlushEmbeddedGeometry;

		// UV Commands
		TSharedPtr< FUICommandInfo > AutoUV;
		
		// Material Commands
		TSharedPtr< FUICommandInfo > Materials;

		// Fracture Commands
		TSharedPtr< FUICommandInfo > Uniform;
		TSharedPtr< FUICommandInfo > Radial;
		TSharedPtr< FUICommandInfo > Clustered;
		TSharedPtr< FUICommandInfo > CustomVoronoi;
		TSharedPtr< FUICommandInfo > Planar;
		TSharedPtr< FUICommandInfo > Slice;
		TSharedPtr< FUICommandInfo > Brick;
		TSharedPtr< FUICommandInfo > Texture;
		TSharedPtr< FUICommandInfo > Mesh;

		// Cleanup Commands
		TSharedPtr< FUICommandInfo > RecomputeNormals;
		TSharedPtr< FUICommandInfo > Resample;
		TSharedPtr< FUICommandInfo > ConvertToMesh;
		TSharedPtr< FUICommandInfo > Validate;
		TSharedPtr< FUICommandInfo > MakeConvex;
		TSharedPtr< FUICommandInfo > Proximity;
		TSharedPtr< FUICommandInfo > FixTinyGeo;
		TSharedPtr< FUICommandInfo > SplitIslands;

		// Property Commands
		TSharedPtr< FUICommandInfo > SetInitialDynamicState;
		TSharedPtr< FUICommandInfo > SetRemoveOnBreak;
		
};

