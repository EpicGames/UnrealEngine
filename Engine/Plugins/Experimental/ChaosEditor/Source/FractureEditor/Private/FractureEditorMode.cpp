// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEditorMode.h"
#include "FractureEditorModeToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "Framework/Application/SlateApplication.h"
#include "EditorModeManager.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "FractureTool.h"
#include "FractureToolUniform.h"
#include "GameFramework/Actor.h"
#include "Engine/Selection.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionHitProxy.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "FractureSelectionTools.h"
#include "EditorViewportClient.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Snapping/ModelingSceneSnappingManager.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "SLevelViewport.h"
#include "FractureSettings.h"

#include "UnrealEdGlobals.h"
#include "EditorModeManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEditorMode)

#define LOCTEXT_NAMESPACE "FFractureEditorModeToolkit"

const FEditorModeID UFractureEditorMode::EM_FractureEditorModeId = TEXT("EM_FractureEditorMode");

UFractureEditorMode::UFractureEditorMode()
{
	Info = FEditorModeInfo(
		EM_FractureEditorModeId,
		LOCTEXT("FractureToolsEditorModeName", "Fracture"),
		FSlateIcon("FractureEditorStyle", "LevelEditor.FractureMode", "LevelEditor.FractureMode.Small"),
		true,
		6000);
}

UFractureEditorMode::~UFractureEditorMode()
{

}

void UFractureEditorMode::Enter()
{
	Super::Enter();

	GEditor->RegisterForUndo(this);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnActorSelectionChanged().AddUObject(this, &UFractureEditorMode::OnActorSelectionChanged);

	FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &UFractureEditorMode::HandlePackageReloaded);

	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());

	// register snapping manager
	UE::Geometry::RegisterSceneSnappingManager(GetInteractiveToolsContext());
	
	// Get initial geometry component selection from currently selected actors when we enter the mode
	USelection* SelectedActors = GEditor->GetSelectedActors();

	TArray<UObject*> SelectedObjects;
	SelectedActors->GetSelectedObjects(SelectedObjects);

	if (FFractureEditorModeToolkit* FractureToolkit = static_cast<FFractureEditorModeToolkit*>(Toolkit.Get()))
	{
		FractureToolkit->SetInitialPalette();
		FractureToolkit->OnHideUnselectedChanged();
	}
	
	OnActorSelectionChanged(SelectedObjects, false);

	ConfigureRealTimeViewportsOverride(true);

}

void UFractureEditorMode::Exit()
{
	GEditor->UnregisterForUndo(this);

	// TODO: cannot deregister currently because if another mode is also registering, its Enter()
	// will be called before our Exit(); add the below line back after this bug is fixed
	//FractureGizmoHelper->DeregisterGizmosWithManager(ToolsContext->ToolManager);

	UE::Geometry::DeregisterSceneSnappingManager(GetInteractiveToolsContext());

	// Empty the geometry component selection set
	TArray<UObject*> SelectedObjects;
	OnActorSelectionChanged(SelectedObjects, false);

	if (Toolkit.IsValid())
	{
		static_cast<FFractureEditorModeToolkit*>(Toolkit.Get())->Shutdown();
	}

	FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	if (LevelEditor)
	{
		LevelEditor->OnActorSelectionChanged().RemoveAll(this);
		LevelEditor->OnMapChanged().RemoveAll( this );
	}

	ConfigureRealTimeViewportsOverride(false);

	// Call base Exit method to ensure proper cleanup
	Super::Exit();
}

bool UFractureEditorMode::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject *, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	if (InContext.Context == FractureTransactionContexts::SelectBoneContext)
	{
		return true;
	}
	
	// TODO: Rigorously use transaction contexts for fracture-related transactions, so we can filter them here
	// Once that's done, we can detect the fracture contexts above and change this last line back to a "return false;"
	return true;
}

void UFractureEditorMode::PostUndo(bool bSuccess)
{
	OnUndoRedo();
}

void UFractureEditorMode::PostRedo(bool bSuccess)
{
	OnUndoRedo();
}

void UFractureEditorMode::ConfigureRealTimeViewportsOverride(bool bEnable)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_FractureMode", "Fracture Mode");
				if (bEnable)
				{
					Viewport.AddRealtimeOverride(bEnable, SystemDisplayName);
				}
				else
				{
					Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
				}
			}
		}
	}
}


void UFractureEditorMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	Super::Tick(ViewportClient, DeltaTime);

	if (!Toolkit.IsValid())
	{
		return;
	}

	// TODO: show a warning if realtime tick is not enabled?

	FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();

	if (FractureToolkit->IsCachedOutlinerGeometryStale(SelectedGeometryComponents))
	{
		// Sanitize bone selections
		for (TWeakObjectPtr<UGeometryCollectionComponent> SelectedCompWeakPtr : SelectedGeometryComponents)
		{
			if (UGeometryCollectionComponent* SelectedComp = SelectedCompWeakPtr.Get())
			{
				FScopedColorEdit Edit(SelectedComp);
				Edit.Sanitize();
			}
		}
		// refresh and also update the cache
		RefreshOutlinerWithCurrentSelection();
		// if we have an active tool, relaunch it to ensure any visualizations are reset
		if (UFractureModalTool* ActiveTool = FractureToolkit->GetActiveTool())
		{
			FractureToolkit->ShutdownActiveTool();
			FractureToolkit->SetActiveTool(ActiveTool);
		}
	}

	if (UFractureModalTool* FractureTool = FractureToolkit->GetActiveTool())
	{
		FractureTool->OnTick(DeltaTime);
	}
}

void UFractureEditorMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	Super::Render(View, Viewport, PDI);

	FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();
 
	if (UFractureModalTool* FractureTool = FractureToolkit->GetActiveTool())
	{
		FractureTool->Render(View, Viewport, PDI);
	}

	RenderSelectionAsBoxes(View, Viewport, PDI);

	// if highlight is not selected disable them
	// technically we should prevent the highlight to be set originally but too many calling site, will need some refactor
	if (const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>())
	{
		if (FractureSettings->SelectionDisplayMode != EFractureSelectionDisplayMode::Highlight)
		{
			for (TWeakObjectPtr<UGeometryCollectionComponent> SelectedCompWeakPtr : SelectedGeometryComponents)
			{
				if (UGeometryCollectionComponent* SelectedComp = SelectedCompWeakPtr.Get())
				{
					FScopedColorEdit EditBoneColor = SelectedComp->EditBoneSelection();
					EditBoneColor.ResetHighlightedBones();
				}
			}
		}
	}
}

void UFractureEditorMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	Super::DrawHUD(ViewportClient, Viewport, View, Canvas);
	FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();

	if (UFractureModalTool* FractureTool = FractureToolkit->GetActiveTool())
	{
		FractureTool->DrawHUD(ViewportClient, Viewport, View, Canvas);
	}
}

void UFractureEditorMode::RenderSelectionAsBoxes(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	// this is a suggestion has nanite based geometry collection may want to override it 
	bool bShouldDisplayBounds = false;
	if (const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>())
	{
		// early exit if the None option has been selected
		if (FractureSettings->SelectionDisplayMode == EFractureSelectionDisplayMode::None)
		{
			return;
		}

		bShouldDisplayBounds = (FractureSettings->SelectionDisplayMode == EFractureSelectionDisplayMode::BoundingBox);
	}

	for (TWeakObjectPtr<UGeometryCollectionComponent> SelectedCompWeakPtr : SelectedGeometryComponents)
	{
		if (UGeometryCollectionComponent* SelectedComp = SelectedCompWeakPtr.Get())
		{
			TSharedPtr<const FGeometryCollection> Collection = (SelectedComp->GetRestCollection()) ? SelectedComp->GetRestCollection()->GetGeometryCollection() : nullptr;
			if (!Collection)
			{
				continue;
			}

			const bool bUsesNanite = SelectedComp->GetSceneProxy() && SelectedComp->GetSceneProxy()->IsNaniteMesh();
			if (!bShouldDisplayBounds && !bUsesNanite)
			{
				continue;
			}

			const GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformFacade(*Collection);
			const GeometryCollection::Facades::FBoundsFacade BoundsFacade(*Collection);
			if (!TransformFacade.IsValid() || !BoundsFacade.IsValid())
			{
				continue;
			}

			const FTransform& WorldTransform = SelectedComp->GetComponentToWorld();
			const TArray<FTransform3f>& ComponentSpaceTransforms = SelectedComp->GetComponentSpaceTransforms3f();

			const int32 NumTransforms = Collection->NumElements(FGeometryCollection::TransformGroup);
			const TManagedArray<FBox>& GeometryBounds = BoundsFacade.GetBoundingBoxes();
			const TManagedArray<int32>& TransformToGeometry = BoundsFacade.GetTransformToGeometryIndex();

			for (int32 SelectedBone : SelectedComp->GetSelectedBones())
			{
				if (SelectedBone < 0 || SelectedBone >= NumTransforms)
				{
					continue;
				}

				FBox BoneExtent;

				// only leaf bones have geometry bounds we need to aggregate them 
				TArray<int32> LeafBoneIndices;
				TransformFacade.ConvertSelectionToRigidNodes(SelectedBone, LeafBoneIndices);
				for (const int32 LeafBoneIndex : LeafBoneIndices)
				{
					const int32 GeometryIndex = TransformToGeometry[LeafBoneIndex];
					if (GeometryBounds.IsValidIndex(GeometryIndex) && ComponentSpaceTransforms.IsValidIndex(LeafBoneIndex))
					{
						BoneExtent += GeometryBounds[GeometryIndex].TransformBy(FTransform(ComponentSpaceTransforms[LeafBoneIndex]));
					}
				}

				if (!BoneExtent.IsValid)
				{
					continue;
				}

				const FVector BoxCenter = BoneExtent.GetCenter();// +FVector(Offset);
				const FVector BoxHalfSize = BoneExtent.GetExtent();
				const FVector Corners[] =
				{
					WorldTransform.TransformPosition(BoxCenter + FVector(-BoxHalfSize.X, -BoxHalfSize.Y, -BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(+BoxHalfSize.X, -BoxHalfSize.Y, -BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(-BoxHalfSize.X, +BoxHalfSize.Y, -BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(+BoxHalfSize.X, +BoxHalfSize.Y, -BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(-BoxHalfSize.X, -BoxHalfSize.Y, +BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(+BoxHalfSize.X, -BoxHalfSize.Y, +BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(-BoxHalfSize.X, +BoxHalfSize.Y, +BoxHalfSize.Z)),
					WorldTransform.TransformPosition(BoxCenter + FVector(+BoxHalfSize.X, +BoxHalfSize.Y, +BoxHalfSize.Z)),
				};
				// bottom 
				PDI->DrawLine(Corners[0], Corners[1], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[1], Corners[3], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[3], Corners[2], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[2], Corners[0], FLinearColor::Yellow, 1.f, 0.0f);

				// top 
				PDI->DrawLine(Corners[4], Corners[5], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[5], Corners[7], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[7], Corners[6], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[6], Corners[4], FLinearColor::Yellow, 1.f, 0.0f);

				// vertical edges
				PDI->DrawLine(Corners[0], Corners[4], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[1], Corners[5], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[2], Corners[6], FLinearColor::Yellow, 1.f, 0.0f);
				PDI->DrawLine(Corners[3], Corners[7], FLinearColor::Yellow, 1.f, 0.0f);
			}
		}
	}
}

void UFractureEditorMode::CreateToolkit()
{
	if (!Toolkit.IsValid() && UsesToolkits())
	{
		Toolkit = MakeShareable(new FFractureEditorModeToolkit);
	}
}

bool UFractureEditorMode::UsesToolkits() const
{
	return true;
}

bool UFractureEditorMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{

	bool bHandled = false;
	if( Event == IE_Pressed )
	{
		FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
		const TSharedRef<FUICommandList> CommandList = Toolkit.Get()->GetToolkitCommands();
		bHandled = CommandList->ProcessCommandBindings( Key, ModifierKeysState, false );
	}
	return bHandled;
}

bool UFractureEditorMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	return SelectFromClick(HitProxy, Click.IsControlDown(), Click.IsShiftDown());
}

bool UFractureEditorMode::SelectFromClick(HHitProxy* HitProxy, bool bCtrlDown, bool bShiftDown)
{
	if (HitProxy && HitProxy->IsA(HGeometryCollection::StaticGetType()))
	{
		HGeometryCollection* GeometryCollectionProxy = (HGeometryCollection*)HitProxy;

		if (GeometryCollectionProxy->Component)
		{
			int32 BoneIndex = GeometryCollectionProxy->BoneIndex;
			// Switch BoneIndex to match the view level
			int32 ViewLevel = -1;
			if (Toolkit.IsValid())
			{
				ViewLevel = ((FFractureEditorModeToolkit*)Toolkit.Get())->GetLevelViewValue();
				if (ViewLevel > -1)
				{
					const FGeometryCollection* GeometryCollection = GeometryCollectionProxy->Component->RestCollection->GetGeometryCollection().Get();
					BoneIndex = FGeometryCollectionClusteringUtility::GetParentOfBoneAtSpecifiedLevel(
						GeometryCollection, BoneIndex, ViewLevel, true /*bSkipFiltered*/);
				}
			}
			
			TArray<int32> BoneIndices({ BoneIndex });

			GeometryCollectionProxy->Component->Modify();
			FFractureSelectionTools::ToggleSelectedBones(GeometryCollectionProxy->Component, BoneIndices, !(bCtrlDown || bShiftDown), bShiftDown);

			if (Toolkit.IsValid())
			{
				FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();
				FractureToolkit->SetBoneSelection(GeometryCollectionProxy->Component, GeometryCollectionProxy->Component->GetSelectedBones(), true, BoneIndex);
			}

			return true;
		}
	}
	else if (HitProxy && HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
	{
		HInstancedStaticMeshInstance* ISMProxy = ((HInstancedStaticMeshInstance*)HitProxy);
		if (ISMProxy->Component)
		{
			// Get the hit ISMComp's GeometryCollection 
			if (UGeometryCollectionComponent* OwningGC = Cast<UGeometryCollectionComponent>(ISMProxy->Component->GetAttachParent()))
			{
				int32 TransformIdx = OwningGC->EmbeddedIndexToTransformIndex(ISMProxy->Component, ISMProxy->InstanceIndex);
				if (TransformIdx > INDEX_NONE)
				{
					TArray<int32> BoneIndices({ TransformIdx });

					OwningGC->Modify();
					FFractureSelectionTools::ToggleSelectedBones(OwningGC, BoneIndices, !(bCtrlDown || bShiftDown), bShiftDown);

					if (Toolkit.IsValid())
					{
						FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();
						FractureToolkit->SetBoneSelection(OwningGC, OwningGC->GetSelectedBones(), true);
					}
				}
			}

			return true;
		}
	}

	return false;

}


bool UFractureEditorMode::InputAxis(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	// Disable the CTRL+drag behavior of the level editor (translate/rotate the selected object along an axis), since it can easily result in 
	// unintentionally moving objects while CTRL-clicking to select/deselect
	// Note: We still want to allow Ctrl+Alt, which is the FrustumSelect in the editor, but not Ctrl+Shift, which is another axis-aligned translation

	bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	bool bAltDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	if (bCtrlDown && !bAltDown)
	{
		return true;
	}
	return UBaseLegacyWidgetEdMode::InputAxis(InViewportClient, InViewport, ControllerId, Key, Delta, DeltaTime);
}


bool UFractureEditorMode::BoxSelect(FBox& InBox, bool InSelect /*= true*/)
{
	FConvexVolume BoxVolume(GetVolumeFromBox(InBox));
	return FrustumSelect(BoxVolume, nullptr, InSelect);
}

bool UFractureEditorMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect /*= true*/)
{
	bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;
	bool bSelectedBones = false;

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			bSelectedBones = UpdateSelectionInFrustum(InFrustum, Actor, bStrictDragSelection, false, false) || bSelectedBones;
		}
	}

	return bSelectedBones;
}

bool UFractureEditorMode::UpdateSelectionInFrustum(const FConvexVolume& InFrustum, AActor* Actor, bool bStrictDragSelection, bool bAppend, bool bRemove)
{
	TArray<UGeometryCollectionComponent*, TInlineAllocator<1>> GeometryComponents;
	Actor->GetComponents(GeometryComponents);
	if (GeometryComponents.Num() == 0)
	{
		return false;
	}

	FTransform ActorTransform = Actor->GetTransform();
	FMatrix InvActorMatrix(ActorTransform.ToInverseMatrixWithScale());

	FConvexVolume SelectionFrustum(TransformFrustum(InFrustum, InvActorMatrix));

	TArray<FBox> BoundsPerBone;
	TArray<int32> SelectedBonesArray;
	bool bSelectionWasUpdated = false;

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeometryComponents)
	{
		BoundsPerBone.Reset();
		SelectedBonesArray.Reset();

		GetComponentGlobalBounds(GeometryCollectionComponent, BoundsPerBone);

		for (int32 Idx = 0; Idx < BoundsPerBone.Num(); Idx++)
		{
			const FBox& TransformedBoneBox = BoundsPerBone[Idx];
			if (!TransformedBoneBox.IsValid)
			{
				continue;
			}
			bool bFullyContained = false;
			bool bIntersected = SelectionFrustum.IntersectBox(TransformedBoneBox.GetCenter(), TransformedBoneBox.GetExtent(), bFullyContained);
			if (bIntersected)
			{
				if (!bStrictDragSelection || (bFullyContained && bStrictDragSelection))
				{
					SelectedBonesArray.Add(Idx);
				}
			}
		}

		bool bNeedsUpdate = UpdateSelection(GeometryCollectionComponent->GetSelectedBones(), SelectedBonesArray, bAppend, bRemove);
		if (bNeedsUpdate)
		{
			FScopedColorEdit ColorEdit = GeometryCollectionComponent->EditBoneSelection();
			ColorEdit.SelectBones(GeometryCollection::ESelectionMode::None);
			ColorEdit.SetSelectedBones(SelectedBonesArray);
			ColorEdit.FilterSelectionToLevel(/*bPreferLowestOnly*/true);
			ColorEdit.SetHighlightedBones(ColorEdit.GetSelectedBones(), true);

			FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();

			if (FractureToolkit)
			{
				FractureToolkit->SetBoneSelection(GeometryCollectionComponent, ColorEdit.GetSelectedBones(), true);
			}

			bSelectionWasUpdated = true;
		}
	}

	return bSelectionWasUpdated;
}

bool UFractureEditorMode::UpdateSelection(const TArray<int32>& PreviousSelection, TArray<int32>& SelectedBonesArray, bool bAppend, bool bRemove)
{
	if (SelectedBonesArray.Num() == 0)
	{
		return false;
	}

	if (!bAppend && !bRemove)
	{
		return true;
	}

	TSet<int32> UpdateSelection(PreviousSelection);
	if (bAppend)
	{
		bool bSelectionChanged = false;
		for (int32 Bone : SelectedBonesArray)
		{
			bool bAlreadyHad = false;
			UpdateSelection.Add(Bone, &bAlreadyHad);
			if (bRemove && bAlreadyHad) // bRemove && bAppend == toggle bones
			{
				UpdateSelection.Remove(Bone);
				bSelectionChanged = true;
			}
			else if (!bAlreadyHad)
			{
				bSelectionChanged = true;
			}
		}
		if (!bSelectionChanged)
		{
			return false;
		}
	}
	else if (bRemove)
	{
		int32 NumRemoved = 0;
		for (int32 Bone : SelectedBonesArray)
		{
			NumRemoved += UpdateSelection.Remove(Bone);
		}
		if (NumRemoved == 0)
		{
			return false; // no update actually needed
		}
	}
	SelectedBonesArray = UpdateSelection.Array();

	return true;
}

bool UFractureEditorMode::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const
{
	UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(PrimitiveComponent);
	if (GeometryCollectionComponent && GeometryCollectionComponent->GetSelectedBones().Num() > 0 && SelectedGeometryComponents.Contains(GeometryCollectionComponent))
	{
		TArray<FBox> BoundsPerBone;
		GetComponentGlobalBounds(GeometryCollectionComponent, BoundsPerBone);

		FBox TotalBoneBox(ForceInit);
		for (int32 BoneIndex : GeometryCollectionComponent->GetSelectedBones())
		{
			TotalBoneBox += BoundsPerBone[BoneIndex];
		}

		InOutBox += TotalBoneBox.TransformBy(GeometryCollectionComponent->GetComponentToWorld());;

		CustomOrbitPivot = InOutBox.GetCenter();
		return true;
	}

	return false;
}

bool UFractureEditorMode::GetPivotForOrbit(FVector& OutPivot) const
{
	if (CustomOrbitPivot.IsSet())
	{
		OutPivot = CustomOrbitPivot.GetValue();
	}

	return CustomOrbitPivot.IsSet();
}

void UFractureEditorMode::OnUndoRedo()
{
	RefreshOutlinerWithCurrentSelection(); // always refresh the outliner in case the geometry collection bones have changed
	for (TWeakObjectPtr<UGeometryCollectionComponent> SelectedCompWeakPtr : SelectedGeometryComponents)
	{
		if (UGeometryCollectionComponent* SelectedComp = SelectedCompWeakPtr.Get())
		{
			// We need to update the bone colors to account for undoing/redoing selection
			bool bForce = true;
			FScopedColorEdit Edit(SelectedComp, bForce);
		}
	}
}

void UFractureEditorMode::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	CustomOrbitPivot.Reset();
	TSet<UGeometryCollectionComponent*> NewGeomSelection;

	int32 ViewLevel = -1;
	if(Toolkit.IsValid())
	{
		ViewLevel = ((FFractureEditorModeToolkit*)Toolkit.Get())->GetLevelViewValue();
	}
	

	// Build new selection set
	for (UObject* ActorObj : NewSelection)
	{
		AActor* Actor = CastChecked<AActor>(ActorObj);
		TArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
		Actor->GetComponents(GeometryCollectionComponents);
		
		for(UGeometryCollectionComponent* GeometryCollectionComponent : GeometryCollectionComponents)
		{
			GeometryCollectionComponent->SetEmbeddedGeometrySelectable(true);
			GeometryCollectionComponent->ForceNativeRendering(true);
			
			FGeometryCollectionEdit RestCollectionEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None);
			if (!RestCollectionEdit.GetRestCollection())
			{
				// guard against the component not having a UGeometryCollection, thsi can happen if the asset has been force deleted while the instance was selected
				continue;
			}

			if (!ensureMsgf(RestCollectionEdit.GetRestCollection()->GetGeometryCollection(), TEXT("UGeometryCollectionComponent had no FGeometryCollection")))
			{
				// guard against the component not having a FGeometryCollection, with ensures because it doesn't seem like this should happen
				continue;
			}
			::GeometryCollection::GenerateTemporaryGuids(RestCollectionEdit.GetRestCollection()->GetGeometryCollection().Get());
			
			constexpr bool bForceUpdate = true; // Force the bone selection and highlight to refresh so bone colors reflect the selection
			FScopedColorEdit ShowBoneColorsEdit(GeometryCollectionComponent, bForceUpdate);
			ShowBoneColorsEdit.SetEnableBoneSelection(true);
			if (ViewLevel != ShowBoneColorsEdit.GetViewLevel())
			{
				ShowBoneColorsEdit.SetLevelViewMode(ViewLevel);
				ShowBoneColorsEdit.FilterSelectionToLevel();
			}
			ShowBoneColorsEdit.Sanitize(); // Clean any stale data (e.g. due to the geometry being edited via a different component)

			NewGeomSelection.Add(GeometryCollectionComponent);
		}
	}

	// reset state for components no longer selected
	for (TWeakObjectPtr<UGeometryCollectionComponent> ExistingSelectionWeakPtr : SelectedGeometryComponents)
	{
		UGeometryCollectionComponent* ExistingSelection = ExistingSelectionWeakPtr.Get();
		if (ExistingSelection && ExistingSelection->IsRegistered() && !ExistingSelection->IsBeingDestroyed() && !NewGeomSelection.Contains(ExistingSelection))
		{
			// This component is no longer selected, clear any modified state

			FScopedColorEdit ShowBoneColorsEdit(ExistingSelection);
			ShowBoneColorsEdit.SetEnableBoneSelection(false);

			ExistingSelection->SetEmbeddedGeometrySelectable(false);
			ExistingSelection->ForceNativeRendering(false);

			// If we have a Hide array on the collection, remove it.
			if (const UGeometryCollection* RestCollection = ExistingSelection->GetRestCollection())
			{
				TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();
				if (GeometryCollection->HasAttribute("Hide", FGeometryCollection::TransformGroup))
				{
					GeometryCollection->RemoveAttribute("Hide", FGeometryCollection::TransformGroup);
				}
			}
			
			ExistingSelection->MarkRenderStateDirty();
		}
	}

	SelectedGeometryComponents.Reset(NewGeomSelection.Num());
	for (UGeometryCollectionComponent* Sel : NewGeomSelection)
	{
		SelectedGeometryComponents.Emplace(Sel);
	}

	RefreshOutlinerWithCurrentSelection();
}

void UFractureEditorMode::RefreshOutlinerWithCurrentSelection()
{
	if (Toolkit.IsValid())
	{
		FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();
		FractureToolkit->SetOutlinerComponents(GetValidSelectedGeometryComponents());
	}
}

TArray<UGeometryCollectionComponent*> UFractureEditorMode::GetValidSelectedGeometryComponents()
{
	TArray<UGeometryCollectionComponent*> Components;
	Components.Reserve(SelectedGeometryComponents.Num());
	for (TWeakObjectPtr<UGeometryCollectionComponent> ComponentWeakPtr : SelectedGeometryComponents)
	{
		if (UGeometryCollectionComponent* Component = ComponentWeakPtr.Get())
		{
			Components.Add(Component);
		}
	}
	return Components;
}


void UFractureEditorMode::GetComponentGlobalBounds(UGeometryCollectionComponent* GeometryCollectionComponent, TArray<FBox>& BoundsPerBone) const
{
	FGeometryCollectionEdit RestCollection = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None);
	UGeometryCollection* GeometryCollection = RestCollection.GetRestCollection();

	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
	FGeometryCollection* OutGeometryCollection = GeometryCollectionPtr.Get();

	const TManagedArray<FTransform3f>& Transform = OutGeometryCollection->Transform;
	const TManagedArray<FBox>& BoundingBox = OutGeometryCollection->BoundingBox;
	const TManagedArray<int32>& TransformToGeometryIndex = OutGeometryCollection->TransformToGeometryIndex;
	const TManagedArray<int32>& Parent = OutGeometryCollection->Parent;

	TManagedArray<FVector3f>* ExplodedVectorsPtr = OutGeometryCollection->FindAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);

	TArray<FTransform> Transforms;
	GeometryCollectionAlgo::GlobalMatrices(Transform, OutGeometryCollection->Parent, Transforms);

	FBox EmptyBounds(EForceInit::ForceInit);
	int32 NumBones = OutGeometryCollection->NumElements(FGeometryCollection::TransformGroup);
	BoundsPerBone.Init(EmptyBounds, NumBones);
	TArray<int32> BoneIndices = GeometryCollectionAlgo::ComputeRecursiveOrder(*OutGeometryCollection);
	for (int32 BoneIdx : BoneIndices)
	{
		int32 GeoIdx = TransformToGeometryIndex[BoneIdx];
		if (GeoIdx != INDEX_NONE)
		{
			const FVector3f& Offset = ExplodedVectorsPtr ? (*ExplodedVectorsPtr)[BoneIdx] : FVector3f::ZeroVector;
			const FBox& Bounds = BoundingBox[TransformToGeometryIndex[BoneIdx]];
			BoundsPerBone[BoneIdx] += Bounds.ShiftBy((FVector)Offset).TransformBy(Transforms[BoneIdx]);
		}
		int32 ParentIdx = Parent[BoneIdx];
		if (ParentIdx != INDEX_NONE)
		{
			BoundsPerBone[ParentIdx] += BoundsPerBone[BoneIdx];
		}
	}
}

void UFractureEditorMode::SelectionStateChanged()
{

}

void UFractureEditorMode::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		// assemble referenced RestCollections
		TMap<const UGeometryCollection*, UGeometryCollectionComponent*> ReferencedRestCollections;
		for (TWeakObjectPtr<UGeometryCollectionComponent> ExistingSelectionWeakPtr : SelectedGeometryComponents)
		{
			if (UGeometryCollectionComponent* ExistingSelection = ExistingSelectionWeakPtr.Get())
			{
				ReferencedRestCollections.Add(TPair<const UGeometryCollection*, UGeometryCollectionComponent*>(ExistingSelection->GetRestCollection(), ExistingSelection));
			}
		}

		// refresh outliner if reloaded package contains a referenced RestCollection
		for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (UGeometryCollection* NewObject = Cast<UGeometryCollection>(RepointedObjectPair.Value))
			{
				if (ReferencedRestCollections.Contains(NewObject))
				{
					if (Toolkit.IsValid())
					{
						FFractureEditorModeToolkit* FractureToolkit = (FFractureEditorModeToolkit*)Toolkit.Get();
						FFractureSelectionTools::ClearSelectedBones(ReferencedRestCollections[NewObject]);
						FractureToolkit->SetOutlinerComponents(GetValidSelectedGeometryComponents());
					}
				}
			}
		}
	}
}

FConvexVolume UFractureEditorMode::TransformFrustum(const FConvexVolume& InFrustum, const FMatrix& InMatrix)
{
	FConvexVolume NewFrustum;
	NewFrustum.Planes.Empty(6);

	for (int32 ii = 0, ni = InFrustum.Planes.Num() ; ii < ni ; ++ii)
	{
		NewFrustum.Planes.Add(InFrustum.Planes[ii].TransformBy(InMatrix));
	}

	NewFrustum.Init();

	return NewFrustum;
}

FConvexVolume UFractureEditorMode::GetVolumeFromBox(const FBox &InBox)
{
	FConvexVolume ConvexVolume;
	ConvexVolume.Planes.Empty(6);

	ConvexVolume.Planes.Add(FPlane(FVector::LeftVector, -InBox.Min.Y));
	ConvexVolume.Planes.Add(FPlane(FVector::RightVector, InBox.Max.Y));
	ConvexVolume.Planes.Add(FPlane(FVector::UpVector, InBox.Max.Z));
	ConvexVolume.Planes.Add(FPlane(FVector::DownVector, -InBox.Min.Z));
	ConvexVolume.Planes.Add(FPlane(FVector::ForwardVector, InBox.Max.X));
	ConvexVolume.Planes.Add(FPlane(FVector::BackwardVector, -InBox.Min.X));

	ConvexVolume.Init();

	return ConvexVolume;
}

#undef LOCTEXT_NAMESPACE



