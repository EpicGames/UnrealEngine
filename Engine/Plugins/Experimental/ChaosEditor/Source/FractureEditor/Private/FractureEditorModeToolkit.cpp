// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEditorModeToolkit.h"
#include "FractureEditorModeToolkit.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorModeManager.h"

#include "Engine/Selection.h"
#include "Dataflow/DataflowInstanceDetails.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "FractureEditorMode.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "IDetailsView.h"
#include "IDetailRootObjectCustomization.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Internationalization/Text.h"

#include "FractureTool.h"
#include "FractureModeSettings.h"

#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"

#include "Styling/AppStyle.h"
#include "Styling/AppStyle.h"
#include "FractureEditor.h"
#include "FractureEditorCommands.h"
#include "FractureEditorStyle.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#include "PlanarCut.h"
#include "FractureToolAutoCluster.h" 
#include "SGeometryCollectionOutliner.h"
#include "SGeometryCollectionHistogram.h"
#include "SGeometryCollectionStatistics.h"
#include "FractureSelectionTools.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#include "Chaos/TriangleMesh.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/MassProperties.h"

#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/STextComboBox.h"

#include "LevelEditor.h"

#include "FractureSettings.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "SPrimaryButton.h"
#include "Widgets/Notifications/SProgressBar.h"

#include "Algo/RemoveIf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureEditorModeToolkit)

#define LOCTEXT_NAMESPACE "FFractureEditorModeToolkit"

namespace FractureEditorModeToolkit::Private
{
	bool bUseNewDataflowEvaluator = true;
	FAutoConsoleVariableRef CVARDataflowEnableGraphEval(TEXT("p.FractureEditor.UseNewDataflowEvaluator"), bUseNewDataflowEvaluator,
		TEXT("Enable new dataflow evaluator for the fracture editor. [def:true]"));
}

TArray<UClass*> FindFractureToolClasses()
{
	TArray<UClass*> Classes;

	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UFractureActionTool::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			Classes.Add(*ClassIterator);
		}
	}

	return Classes;
}

FFractureViewSettingsCustomization::FFractureViewSettingsCustomization(FFractureEditorModeToolkit* InToolkit) 
	: Toolkit(InToolkit)
{

}

TSharedRef<IDetailCustomization> FFractureViewSettingsCustomization::MakeInstance(FFractureEditorModeToolkit* InToolkit)
{
	return MakeShareable(new FFractureViewSettingsCustomization(InToolkit));
}

void 
FFractureViewSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) 
{
	IDetailCategoryBuilder& ViewCategory = DetailBuilder.EditCategory("ViewSettings", FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	TSharedRef<IPropertyHandle> LevelProperty = DetailBuilder.GetProperty("FractureLevel");
	
	ViewCategory.AddProperty(LevelProperty)
	.CustomWidget()
	.NameContent()
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ) )
		.Text(LevelProperty->GetPropertyDisplayName())
	]
	.ValueContent()
	[
		SNew( SComboButton)
		.ContentPadding(0)
		.OnGetMenuContent(Toolkit, &FFractureEditorModeToolkit::GetLevelViewMenuContent, LevelProperty) 
		.ButtonContent()
		[
			SNew(STextBlock)
			.Justification(ETextJustify::Left)
			.Text_Lambda( [=]() -> FText {

				int32 FractureLevel = 5;
				LevelProperty->GetValue(FractureLevel);

				if (FractureLevel < 0)
				{
					return LOCTEXT("FractureViewAllLevels", "All");
				}
				else if (FractureLevel == 0)
				{
					return LOCTEXT("FractureViewRootLevel", "Root");
				}

				return FText::Format(NSLOCTEXT("FractureEditor", "CurrentLevel", "{0}"), FText::AsNumber(FractureLevel));

			})
		]
	];
};	

FHistogramSettingsCustomization::FHistogramSettingsCustomization(FFractureEditorModeToolkit* InToolkit)
	: Toolkit(InToolkit)
{

}

TSharedRef<IDetailCustomization> FHistogramSettingsCustomization::MakeInstance(FFractureEditorModeToolkit* InToolkit)
{
	return MakeShareable(new FHistogramSettingsCustomization(InToolkit));
}

void
FHistogramSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
};

FOutlinerSettingsCustomization::FOutlinerSettingsCustomization(FFractureEditorModeToolkit* InToolkit)
	: Toolkit(InToolkit)
{

}

TSharedRef<IDetailCustomization> FOutlinerSettingsCustomization::MakeInstance(FFractureEditorModeToolkit* InToolkit)
{
	return MakeShareable(new FOutlinerSettingsCustomization(InToolkit));
}

void
FOutlinerSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
};



FFractureEditorModeToolkit::FFractureEditorModeToolkit()
	: ActiveTool(nullptr)
{
}

FFractureEditorModeToolkit::~FFractureEditorModeToolkit()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		auto& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditorModule.OnMapChanged().RemoveAll(this);
	}
	if (UFractureModeSettings* Settings = GetMutableDefault<UFractureModeSettings>())
	{
		Settings->OnModified.Remove(ProjectSettingsModifiedHandle);
	}
	if (OverridesDetailsView)
	{
		OverridesDetailsView->GetOnFinishedChangingPropertiesDelegate().RemoveAll(this);
	}
}

void FFractureEditorModeToolkit::UpdateAssetLocationMode(TSharedPtr<FString> NewString)
{
	UFractureModeSettings* Settings = GetMutableDefault<UFractureModeSettings>();
	EFractureModeNewAssetLocation NewAssetLocation = EFractureModeNewAssetLocation::SourceAssetFolder;
	if (NewString == AssetLocationModes[0])
	{
		NewAssetLocation = EFractureModeNewAssetLocation::SourceAssetFolder;
	}
	else if (NewString == AssetLocationModes[1])
	{
		NewAssetLocation = EFractureModeNewAssetLocation::LastUsedFolder;
	}
	else if (NewString == AssetLocationModes[2])
	{
		NewAssetLocation = EFractureModeNewAssetLocation::ContentBrowserFolder;
	}

	Settings->NewAssetLocation = NewAssetLocation;
	Settings->SaveConfig();
}

void FFractureEditorModeToolkit::UpdateAssetPanelFromSettings()
{
	const UFractureModeSettings* Settings = GetDefault<UFractureModeSettings>();

	switch (Settings->NewAssetLocation)
	{
	case EFractureModeNewAssetLocation::ContentBrowserFolder:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[2]);
		break;
	case EFractureModeNewAssetLocation::LastUsedFolder:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[1]);
		break;
	case EFractureModeNewAssetLocation::SourceAssetFolder:
	default:
		AssetLocationMode->SetSelectedItem(AssetLocationModes[0]);
		break;
	}
}

void FFractureEditorModeToolkit::CreateVariableOverrideDetailView()
{
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
		DetailsViewArgs.bShowKeyablePropertiesOption = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	OverridesDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	auto MakeDataflowInstanceDetailCustomizationLambda = []()
		{
			constexpr bool bOnlyShowVariableOverrides = true;
			return MakeShared<FDataflowInstanceDetailCustomization>(bOnlyShowVariableOverrides);
		};
	FOnGetDetailCustomizationInstance DataflowInstanceDetailsCustomizationInstance = FOnGetDetailCustomizationInstance::CreateLambda(MakeDataflowInstanceDetailCustomizationLambda);
	OverridesDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyLayout(FDataflowInstance::StaticStruct(), DataflowInstanceDetailsCustomizationInstance);
	OnFinishedChangingVariableOverridesDelegateHandle = OverridesDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &FFractureEditorModeToolkit::OnPropertyValueChanged);

	RefreshVariableOverrideDetailView(nullptr);
}

void FFractureEditorModeToolkit::OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>())
	{
		if (TStrongObjectPtr<const UGeometryCollection> RestCollection = FractureSettings->RestCollection.Pin())
		{
			// we need a modifiable object
			if (UGeometryCollection* EditableRestCollection = const_cast<UGeometryCollection*>(RestCollection.Get()))
			{
				// Recreate the context for now
 				DataflowContext = MakeShared<UE::Dataflow::FEngineContext>(EditableRestCollection);
			}
		}
	}
}

void FFractureEditorModeToolkit::RefreshVariableOverrideDetailView(const UGeometryCollection* RestCollection)
{
	if (RestCollection == nullptr)
	{
		OverridesDetailsView->GetDetailsView()->SetObject(nullptr);
		OverridesDetailsView->SetStructureData(nullptr);
		return;
	}
	// we only get a const collection from the component, but the fracture editor needs to be able to edit the variables 
	UGeometryCollection* EditableRestCollection = const_cast<UGeometryCollection*>(RestCollection);
	if (const IDataflowInstanceInterface* Interface = EditableRestCollection)
	{
		OverridesDetailsView->GetDetailsView()->SetObject(EditableRestCollection);
		TSharedPtr<FStructOnScope> StructOnScope = Interface->GetDataflowInstance().MakeStructOnScope();
		OverridesDetailsView->SetStructureData(StructOnScope);

		// Recreate the context 
		if (!DataflowContext.IsValid() || !IsValid(DataflowContext->Owner) || DataflowContext->Owner != EditableRestCollection)
		{
			DataflowContext = MakeShared<UE::Dataflow::FEngineContext>(EditableRestCollection);
		}
	}
}

FReply FFractureEditorModeToolkit::OnDataflowOverridesUpdateAsset()
{
	// get the displayed asset from the settings
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	if (FractureSettings && FractureSettings->RestCollection.IsValid())
	{
		if (FractureEditorModeToolkit::Private::bUseNewDataflowEvaluator)
		{
			/*auto OnEvaluationCompleted = [DataflowContext = DataflowContext](UE::Dataflow::FContext&)
				{
					if (DataflowContext)
					{
						DataflowContext->UpdateAssetFromTerminalNodes(true);
					}
				};*/
			DataflowEvaluator.Reset();
			if (DataflowContext)
			{
				DataflowEvaluator = DataflowContext->EvaluateGraph<UE::Dataflow::FDataflowTaskGraphEvaluator>(/*OnEvaluationCompleted*/{});
				if (!DataflowEvaluator.IsValid())
				{
					UE_LOGF(LogFractureTool, Error, "FractureMode: Failed to start evaluating Dataflow graph for geometry collection %ls", *FractureSettings->RestCollection->GetFName().ToString());
				}
			}
		}
		else
		{
			FractureSettings->RestCollection->GetDataflowInstance().UpdateOwnerAsset(/* bUpdateDependentAssets */true);
		}
	}
	return FReply::Handled();
}

bool FFractureEditorModeToolkit::CanDataflowOverridesUpdateAsset() const
{
	return true;
}

bool FFractureEditorModeToolkit::GetDataflowOverridesUpdateAssetEnabled() const
{
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	if (FractureSettings && FractureSettings->RestCollection.IsValid())
	{
		if (FractureSettings->RestCollection->GetDataflowInstance().GetDataflowAsset())
		{
			return true;
		}
	}
	return false;
}

void FFractureEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FFractureEditorModule& FractureModule = FModuleManager::GetModuleChecked<FFractureEditorModule>("FractureEditor");

	const FFractureEditorCommands& Commands = FFractureEditorCommands::Get();

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FFractureEditorModeToolkit::OnObjectPostEditChange);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnMapChanged().AddRaw(this, &FFractureEditorModeToolkit::HandleMapChanged);

	BeginPIEDelegateHandle = FEditorDelegates::BeginPIE.AddLambda([this](bool bSimulating)
	{
		SetActiveTool(nullptr);
	});

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;

	DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	FractureSettingsDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	EditModule.RegisterCustomClassLayout("FractureSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FFractureViewSettingsCustomization::MakeInstance, this));

	TArray<UObject*> Settings;
	Settings.Add(GetMutableDefault<UFractureSettings>());
	FractureSettingsDetailsView->SetObjects(Settings);

	HistogramDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	EditModule.RegisterCustomClassLayout("HistogramSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FHistogramSettingsCustomization::MakeInstance, this));
	HistogramDetailsView->SetObject(GetMutableDefault<UHistogramSettings>());

	OutlinerDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	EditModule.RegisterCustomClassLayout("OutlinerSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FOutlinerSettingsCustomization::MakeInstance, this));
	OutlinerDetailsView->SetObject(GetMutableDefault<UOutlinerSettings>());

	CreateVariableOverrideDetailView();
	CreateDataflowProgressBar();

	AssetLocationModes.Reset();
	AssetLocationModes.Add(MakeShared<FString>(TEXT("Source Asset Folder")));
	AssetLocationModes.Add(MakeShared<FString>(TEXT("Last Used Folder")));
	AssetLocationModes.Add(MakeShared<FString>(TEXT("Content Browser Folder")));
	AssetLocationMode = SNew(STextComboBox)
		.OptionsSource(&AssetLocationModes)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> String, ESelectInfo::Type) { UpdateAssetLocationMode(String); });

	const TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox)
		+ SVerticalBox::Slot().HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetLocationLabel", "New Asset Location"))
				]
				+ SHorizontalBox::Slot().Padding(0).FillWidth(4.0f)
				[
					AssetLocationMode->AsShared()
				]
		];

	TSharedPtr<SExpandableArea> AssetConfigPanel = SNew(SExpandableArea)
		.HeaderPadding(FMargin(0.f))
		.Padding(FMargin(8.f))
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("EditorModesPanel.CategoryFontStyle"))
		.BodyContent()
		[
			Content->AsShared()
		]
		.HeaderContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("FractureSettingsPanelHeader", "Fracture Mode Quick Settings"))
				]

				+ SHorizontalBox::Slot().HAlign(HAlign_Right).VAlign(VAlign_Center).AutoWidth()
				[
					SNew(SComboButton)
						.HasDownArrow(false)
						.MenuPlacement(EMenuPlacement::MenuPlacement_MenuRight)
						.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
						.OnGetMenuContent(FOnGetContent::CreateLambda([this]()
							{
								return MakeMenu_FractureModeConfigSettings();
							}))
						.ContentPadding(FMargin(3.0f, 1.0f))
								.ButtonContent()
								[
									SNew(SImage)
										.Image(FFractureEditorStyle::Get().GetBrush("FractureEditor.DefaultSettings"))
										.ColorAndOpacity(FSlateColor::UseForeground())
								]
				]

		];


	TSharedPtr<SVerticalBox> DataflowOverrideWidget =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				OverridesDetailsView->GetWidget().ToSharedRef()
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			DataflowProgressBar.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0)
			.AutoWidth()
			[
				SNew(SPrimaryButton)
					.OnClicked(this, &FFractureEditorModeToolkit::OnDataflowOverridesUpdateAsset)
					.IsEnabled(this, &FFractureEditorModeToolkit::CanDataflowOverridesUpdateAsset)
					.Text(LOCTEXT("DataflowOverrideUpdateAssetButton", "Update Asset From Dataflow"))
					.IsEnabled_Lambda([this]() -> bool { return GetDataflowOverridesUpdateAssetEnabled(); })
			]
		];

	const TSharedPtr<SVerticalBox> ToolkitWidgetVBox = SNew(SVerticalBox);

	float Padding = 4.0f;
	FMargin MorePadding = FMargin(10.0f, 2.0f);
	SAssignNew(ToolkitWidget, SBorder).HAlign(HAlign_Fill)
		.Padding(4)
		[
			ToolkitWidgetVBox->AsShared()
		];
	ToolkitWidgetVBox->AddSlot().HAlign(HAlign_Fill).FillHeight(1.f)
		[

			SNew(SSplitter)
			.Orientation( Orient_Vertical )
			+SSplitter::Slot()
			.SizeRule( TAttribute<SSplitter::ESizeRule>::Create( [this] () { 
				return (GetActiveTool() != nullptr) ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; 
			} ) )
			.Value(1.f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SScrollBox)
					+SScrollBox::Slot()
					[
						FractureSettingsDetailsView.ToSharedRef()
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					DataflowOverrideWidget.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						DetailsView.ToSharedRef()
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SSpacer)	
					]	

					+SHorizontalBox::Slot()
					.Padding(4.0)
					.AutoWidth()
					[
						SNew(SPrimaryButton)
						.OnClicked(this, &FFractureEditorModeToolkit::OnModalClicked)
						.IsEnabled( this, &FFractureEditorModeToolkit::CanExecuteModal)
						.Text_Lambda( [this] () -> FText { return ActiveTool ? ActiveTool->GetApplyText() :  LOCTEXT("FractureApplyButton", "Apply"); })
						.Visibility_Lambda( [this] () -> EVisibility { return (GetActiveTool() == nullptr) ? EVisibility::Collapsed : EVisibility::Visible; })
					]

					+ SHorizontalBox::Slot()
					.Padding(4.0)
					.AutoWidth()
					[
						SNew(SButton)
						.OnClicked_Lambda( [this] () -> FReply { SetActiveTool(0); return FReply::Handled(); } )
						.Text(FText(LOCTEXT("FractureCancelButton", "Cancel")))
						.Visibility_Lambda( [this] () -> EVisibility { return (GetActiveTool() == nullptr) ? EVisibility::Collapsed : EVisibility::Visible; })
					]
				]
			]
		];
	ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).VAlign(VAlign_Bottom).Padding(0)
		[
			AssetConfigPanel->AsShared()
		];


	// register callback
	UFractureModeSettings* FractureModeSettings = GetMutableDefault<UFractureModeSettings>();
	ProjectSettingsModifiedHandle = FractureModeSettings->OnModified.AddLambda([this](UObject*, FProperty*) { OnProjectSettingsModified(); });

	// initialize combos
	UpdateAssetPanelFromSettings();

	// Bind Chaos Commands;
	BindCommands();

	FModeToolkit::Init(InitToolkitHost, InOwningMode);

}

void FFractureEditorModeToolkit::OnProjectSettingsModified()
{
	UpdateAssetPanelFromSettings();
}

namespace
{

	void MakeFractureQuickSettings(FMenuBuilder& MenuBuilder)
	{
		const FUIAction OpenFractureModeProjectSettings(
			FExecuteAction::CreateLambda([]
				{
					if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
					{
						SettingsModule->ShowViewer("Project", "Plugins", "FractureMode");
					}
				}), FCanExecuteAction(), FIsActionChecked());
		MenuBuilder.AddMenuEntry(LOCTEXT("FractureModeProjectSettings", "Project Settings"),
			LOCTEXT("FractureModeProjectSettings_Tooltip", "Jump to the Project Settings for Fracture Mode. Project Settings are Project-specific."),
			FSlateIcon(), OpenFractureModeProjectSettings, NAME_None, EUserInterfaceActionType::Button);

		const FUIAction OpenFractureModeEditorSettings(
			FExecuteAction::CreateLambda([]
				{
					if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
					{
						SettingsModule->ShowViewer("Editor", "Plugins", "FractureEditor");
					}
				}), FCanExecuteAction(), FIsActionChecked());
		MenuBuilder.AddMenuEntry(LOCTEXT("FractureModeEditorSettings", "Editor Settings"),
			LOCTEXT("FractureModeEditorSettings_Tooltip", "Jump to the Editor Settings for Fracture Mode. Editor Settings apply across all Projects."),
			FSlateIcon(), OpenFractureModeEditorSettings, NAME_None, EUserInterfaceActionType::Button);
	}
}

TSharedRef<SWidget> FFractureEditorModeToolkit::MakeMenu_FractureModeConfigSettings()
{
	FMenuBuilder MenuBuilder(true, TSharedPtr<FUICommandList>());

	MenuBuilder.BeginSection("Section_Settings", LOCTEXT("Section_Settings", "Quick Settings"));
	constexpr bool bQuickSettingsInSubMenu = false;
	if (!bQuickSettingsInSubMenu)
	{
		MakeFractureQuickSettings(MenuBuilder);
	}
	else
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("QuickSettingsSubMenu", "Jump To Settings"), LOCTEXT("QuickSettingsSubMenu_ToolTip", "Jump to sections of the Settings dialogs relevant to Fracture Mode"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				MakeFractureQuickSettings(SubMenuBuilder);
				}));
	}
	MenuBuilder.EndSection();

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}

void FFractureEditorModeToolkit::RequestModeUITabs()
{
	FModeToolkit::RequestModeUITabs();
	if (TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		TSharedPtr<FWorkspaceItem> MenuModeCategoryPtr = ModeUILayerPtr->GetModeMenuCategory();

		if(!MenuModeCategoryPtr)
		{ 
			return;
		}
		HierarchyTabInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FFractureEditorModeToolkit::CreateHierarchyTab);
		HierarchyTabInfo.TabLabel = LOCTEXT("FractureHierarchy", "Fracture Hierarchy");
		HierarchyTabInfo.TabTooltip = LOCTEXT("ModesToolboxTabTooltipText", "Open the  Modes tab, which contains the active editor mode's settings.");
		HierarchyTabInfo.TabIcon = GetEditorModeIcon();
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::TopRightTabID, HierarchyTabInfo);


		StatisticsTabInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FFractureEditorModeToolkit::CreateStatisticsTab);
		StatisticsTabInfo.TabLabel = LOCTEXT("FractureStatistics", "Level Statistics");
		StatisticsTabInfo.TabTooltip = LOCTEXT("ModesToolboxTabTooltipText", "Open the  Modes tab, which contains the active editor mode's settings.");
		StatisticsTabInfo.TabIcon = GetEditorModeIcon();
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::BottomLeftTabID, StatisticsTabInfo);
	}
}

TSharedPtr<SWidget> FFractureEditorModeToolkit::GetInlineContent() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.VAlign(VAlign_Fill)
		[
			ToolkitWidget.ToSharedRef()
		];
}

void FFractureEditorModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	// (Note this logic is from ModelingToolsEditorModeToolkit.cpp, as the modeling tools had the same issue)
	// FModeToolkit::UpdatePrimaryModePanel() wrapped our GetInlineContent() output in a SScrollBar widget,
	// however this doesn't make sense as we want to dock panels to the "top" and "bottom" of our mode panel area,
	// and the details panel in the middle has it's own scrollbar already. The SScrollBar is hardcoded as the content
	// of FModeToolkit::InlineContentHolder so we can just replace it here
	InlineContentHolder->SetContent(GetInlineContent().ToSharedRef());

	if (TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		TSharedPtr<FTabManager> TabManagerPtr = ModeUILayerPtr->GetTabManager();
		if (!TabManagerPtr)
		{
			return;
		}
		HierarchyTab = TabManagerPtr->TryInvokeTab(UAssetEditorUISubsystem::TopRightTabID);
		StatisticsTab = TabManagerPtr->TryInvokeTab(UAssetEditorUISubsystem::BottomLeftTabID);
	}


	//
	// Apply custom section header colors.
	// See comments below, this is done via directly manipulating Slate widgets generated deep inside BaseToolkit.cpp,
	// and will stop working if the Slate widget structure changes
	//

	UFractureModeCustomizationSettings* UISettings = GetMutableDefault<UFractureModeCustomizationSettings>();

	// look up default radii for palette toolbar expandable area headers
	FVector4 HeaderRadii(4, 4, 0, 0);
	const FSlateBrush* BaseBrush = FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader");
	if (BaseBrush != nullptr)
	{
		HeaderRadii = BaseBrush->OutlineSettings.CornerRadii;
	}

	// Generate a map for tool specific colors
	TMap<FString, FLinearColor> SectionIconColorMap;
	TMap<FString, TMap<FString, FLinearColor>> SectionToolIconColorMap;
	for (const FFractureModeCustomToolColor& ToolColor : UISettings->ToolColors)
	{
		FString SectionName, ToolName;
		ToolColor.ToolName.Split(".", &SectionName, &ToolName);
		SectionName.ToLowerInline();
		if (ToolName.Len() > 0)
		{
			if (!SectionToolIconColorMap.Contains(SectionName))
			{
				SectionToolIconColorMap.Emplace(SectionName, TMap<FString, FLinearColor>());
			}
			SectionToolIconColorMap[SectionName].Add(ToolName, ToolColor.Color);
		}
		else
		{
			SectionIconColorMap.Emplace(ToolColor.ToolName.ToLower(), ToolColor.Color);
		}
	}

	for (FEdModeToolbarRow& ToolbarRow : ActiveToolBarRows)
	{
		// Update section header colors
		for (FFractureModeCustomSectionColor ToolColor : UISettings->SectionColors)
		{
			if (ToolColor.SectionName.Equals(ToolbarRow.DisplayName.ToString(), ESearchCase::IgnoreCase)
			 || ToolColor.SectionName.Equals(ToolbarRow.PaletteName.ToString(), ESearchCase::IgnoreCase))
			{
				// code below is highly dependent on the structure of the ToolbarRow.ToolbarWidget. Currently this is 
				// a SMultiBoxWidget, a few levels below a SExpandableArea. The SExpandableArea contains a SVerticalBox
				// with the header as a SBorder in Slot 0. The code will fail gracefully if this structure changes.

				TSharedPtr<SWidget> ExpanderVBoxWidget = (ToolbarRow.ToolbarWidget.IsValid() && ToolbarRow.ToolbarWidget->GetParentWidget().IsValid()) ?
					ToolbarRow.ToolbarWidget->GetParentWidget()->GetParentWidget() : TSharedPtr<SWidget>();
				if (ExpanderVBoxWidget.IsValid() && ExpanderVBoxWidget->GetTypeAsString().Compare(TEXT("SVerticalBox")) == 0)
				{
					TSharedPtr<SVerticalBox> ExpanderVBox = StaticCastSharedPtr<SVerticalBox>(ExpanderVBoxWidget);
					if (ExpanderVBox.IsValid() && ExpanderVBox->NumSlots() > 0)
					{
						const TSharedRef<SWidget>& SlotWidgetRef = ExpanderVBox->GetSlot(0).GetWidget();
						TSharedPtr<SWidget> SlotWidgetPtr(SlotWidgetRef);
						if (SlotWidgetPtr.IsValid() && SlotWidgetPtr->GetTypeAsString().Compare(TEXT("SBorder")) == 0)
						{
							TSharedPtr<SBorder> TopBorder = StaticCastSharedPtr<SBorder>(SlotWidgetPtr);
							if (TopBorder.IsValid())
							{
								TopBorder->SetBorderImage(new FSlateRoundedBoxBrush(FSlateColor(ToolColor.Color), HeaderRadii));
							}
						}
					}
				}
				break;
			}
		}

		// Update tool colors
		FLinearColor* SectionIconColor = SectionIconColorMap.Find(ToolbarRow.PaletteName.ToString().ToLower());
		if (!SectionIconColor)
		{
			SectionIconColor = SectionIconColorMap.Find(ToolbarRow.DisplayName.ToString().ToLower());
		}
		TMap<FString, FLinearColor>* SectionToolIconColors = SectionToolIconColorMap.Find(ToolbarRow.PaletteName.ToString().ToLower());
		if (!SectionToolIconColors)
		{
			SectionToolIconColors = SectionToolIconColorMap.Find(ToolbarRow.DisplayName.ToString().ToLower());
		}
		if (SectionIconColor || SectionToolIconColors)
		{
			// code below is highly dependent on the structure of the ToolbarRow.ToolbarWidget. Currently this is 
			// a SMultiBoxWidget. The code will fail gracefully if this structure changes.
			
			if (ToolbarRow.ToolbarWidget.IsValid() && ToolbarRow.ToolbarWidget->GetTypeAsString().Compare(TEXT("SMultiBoxWidget")) == 0)
			{
				auto FindFirstChildWidget = [](const TSharedPtr<SWidget>& Widget, const FString& WidgetType)
				{
					auto FindFirstChildWidgetImpl = [](const TSharedPtr<SWidget>& Widget, const FString& WidgetType, auto& FindRef) -> TSharedPtr<SWidget>
					{
						TSharedPtr<SWidget> Result;
						if (Widget.IsValid())
						{
							FChildren* Children = Widget->GetChildren();
							const int32 NumChild = Children ? Children->NumSlot() : 0;
							for (int32 ChildIdx = 0; ChildIdx < NumChild; ++ChildIdx)
							{
								const TSharedRef<SWidget> ChildWidgetRef = Children->GetChildAt(ChildIdx);
								TSharedPtr<SWidget> ChildWidgetPtr(ChildWidgetRef);
								if (ChildWidgetPtr.IsValid())
								{
									if (ChildWidgetPtr->GetTypeAsString().Compare(WidgetType) == 0)
									{
										Result = ChildWidgetPtr;
										break;
									}

									Result = FindRef(ChildWidgetPtr, WidgetType, FindRef);
									if (Result.IsValid())
									{
										break;
									}
								}
							}
						}
						return Result;
					};
					return FindFirstChildWidgetImpl(Widget, WidgetType, FindFirstChildWidgetImpl);
				};

				TSharedPtr<SWidget> PanelWidget = FindFirstChildWidget(ToolbarRow.ToolbarWidget, TEXT("SUniformWrapPanel"));
				if (PanelWidget.IsValid())
				{
					// This contains each of the FToolBarButtonBlock items for this row.
					FChildren* PanelChildren = PanelWidget->GetChildren();
					const int32 NumChild = PanelChildren ? PanelChildren->NumSlot() : 0;
					for (int32 ChildIdx = 0; ChildIdx < NumChild; ++ChildIdx)
					{
						const TSharedRef<SWidget> ChildWidgetRef = PanelChildren->GetChildAt(ChildIdx);
						TSharedPtr<SWidget> ChildWidgetPtr(ChildWidgetRef);
						if (ChildWidgetPtr.IsValid() && ChildWidgetPtr->GetTypeAsString().Compare(TEXT("SToolBarButtonBlock")) == 0)
						{
							TSharedPtr<SToolBarButtonBlock> ToolBarButton = StaticCastSharedPtr<SToolBarButtonBlock>(ChildWidgetPtr);
							if (ToolBarButton.IsValid())
							{
								TSharedPtr<SWidget> LayeredImageWidget = FindFirstChildWidget(ToolBarButton, TEXT("SLayeredImage"));
								TSharedPtr<SWidget> TextBlockWidget = FindFirstChildWidget(ToolBarButton, TEXT("STextBlock"));
								if (LayeredImageWidget.IsValid() && TextBlockWidget.IsValid())
								{
									TSharedPtr<SImage> ImageWidget = StaticCastSharedPtr<SImage>(LayeredImageWidget);
									TSharedPtr<STextBlock> TextWidget = StaticCastSharedPtr<STextBlock>(TextBlockWidget);
									// Check if this Section.Tool has an explicit color entry. If not, fallback
									// to any Section-wide color entry, otherwise leave the tint alone.
									FLinearColor* TintColor = SectionToolIconColors ? SectionToolIconColors->Find(TextWidget->GetText().ToString()) : nullptr;
									if (!TintColor)
									{
										const FString* SourceText = FTextInspector::GetSourceString(TextWidget->GetText());
										TintColor = SectionToolIconColors && SourceText ? SectionToolIconColors->Find(*SourceText) : nullptr;
										if (!TintColor)
										{
											TintColor = SectionIconColor;
										}
									}
									if (TintColor)
									{
										ImageWidget->SetColorAndOpacity(FSlateColor(*TintColor));
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// TODO: This is a workaround for the Mode's Enter() call happening *before* the toolkit UI-building call.  Ideally we'd find a better way of making sure the toolkit UI knows about the current selection.
	UFractureEditorMode* Mode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (Mode)
	{
		Mode->RefreshOutlinerWithCurrentSelection();
	}
}

TSharedRef<SDockTab> FFractureEditorModeToolkit::CreateHierarchyTab(const FSpawnTabArgs& Args)
{
	float Padding = 4.0f;
	FMargin MorePadding = FMargin(10.0f, 2.0f);

	TSharedRef<SExpandableArea> HistogramExpander = SNew(SExpandableArea)
		.AreaTitle(FText(LOCTEXT("Histogram", "Histogram")))
		.HeaderPadding(FMargin(2.0, 2.0))
		.Padding(MorePadding)
		.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
		.BodyBorderBackgroundColor(FLinearColor(1.0, 0.0, 0.0))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("HistogramDetailsView.CategoryFontStyle"))
		.InitiallyCollapsed(true)
		.Clipping(EWidgetClipping::ClipToBounds)
		.BodyContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				HistogramDetailsView.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(HistogramView, SGeometryCollectionHistogram)
				.OnBoneSelectionChanged(this, &FFractureEditorModeToolkit::OnHistogramBoneSelectionChanged)
			]
		];

	TSharedRef<SExpandableArea> OutlinerExpander = SNew(SExpandableArea)
		.AreaTitle(FText(LOCTEXT("Outliner", "Outliner")))
		.HeaderPadding(FMargin(2.0, 2.0))
		.Padding(MorePadding)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
		.BodyBorderBackgroundColor(FLinearColor(1.0, 0.0, 0.0))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		.BodyContent()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
			
				+ SSplitter::Slot()
				.SizeRule(TAttribute<SSplitter::ESizeRule>::Create([this, HistogramExpander]() {
					return HistogramExpander->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
				}))
				.Value(1.f)
				[
					HistogramExpander
				]

				+ SSplitter::Slot()
				.SizeRule(SSplitter::ESizeRule::FractionOfParent)
				.Value(1.f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						OutlinerDetailsView.ToSharedRef()
					]
					+ SVerticalBox::Slot()
					[
						SAssignNew(OutlinerView, SGeometryCollectionOutliner)
						.OnBoneSelectionChanged(this, &FFractureEditorModeToolkit::OnOutlinerBoneSelectionChanged)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
						.ContentPadding(FMargin(2, 0))
						.HAlign(HAlign_Center)
						.OnClicked(this, &FFractureEditorModeToolkit::OnRefreshOutlinerButtonClicked)
						.Text(LOCTEXT("GCOUtliner_Refresh_Button_Text", "Refresh"))
						.ToolTipText(LOCTEXT("GCOUtliner_Refresh_Button_ToolTip", "Refresh the outliner"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(this, &FFractureEditorModeToolkit::GetSelectionInfo)
					]
				]
			]
		];

	TSharedPtr<SDockTab> CreatedTab = SNew(SDockTab)
		[
			OutlinerExpander
		];

	HierarchyTab = CreatedTab;
	return CreatedTab.ToSharedRef();
}

FReply FFractureEditorModeToolkit::OnRefreshOutlinerButtonClicked()
{
	RefreshOutliner();
	return FReply::Handled();
}

void FFractureEditorModeToolkit::RefreshOutliner()
{
	if (OutlinerView)
	{
		OutlinerView->RegenerateItems();
	}
}

TSharedRef<SDockTab> FFractureEditorModeToolkit::CreateStatisticsTab(const FSpawnTabArgs& Args)
{
	FMargin MorePadding = FMargin(10.0f, 2.0f);
	TSharedRef<SExpandableArea> StatisticsExpander = SNew(SExpandableArea)
		.AreaTitle(FText(LOCTEXT("LevelStatistics", "Level Statistics")))
		.HeaderPadding(FMargin(2.0, 2.0))
		.Padding(MorePadding)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
		.BodyBorderBackgroundColor(FLinearColor(1.0, 0.0, 0.0))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		.BodyContent()
		[
			SAssignNew(StatisticsView, SGeometryCollectionStatistics)
			//SNew(STextBlock)
			//.Text(LOCTEXT("Statt", "Statt")) //this, &FFractureEditorModeToolkit::GetStatisticsSummary)
		];
	TSharedPtr<SDockTab> CreatedTab = SNew(SDockTab)
		[
			StatisticsExpander
		];

	StatisticsTab = CreatedTab;
	return CreatedTab.ToSharedRef();
}

void FFractureEditorModeToolkit::SetOutlinerColumnMode(EOutlinerColumnMode ColumnMode)
{
	UOutlinerSettings* OutlinerSettings = GetMutableDefault<UOutlinerSettings>();
	OutlinerSettings->ColumnMode = ColumnMode;
	UpdateOutlinerHeader();
}

void FFractureEditorModeToolkit::UpdateOutlinerHeader()
{
	OutlinerView->RegenerateHeader();
	FGeometryCollectionStatistics Stats;
	GetStatisticsSummary(Stats);
	StatisticsView->SetStatistics(Stats);
}

void FFractureEditorModeToolkit::OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent )
{
	if (PropertyChangedEvent.Property)
	{
		if ( PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFractureSettings, ExplodeAmount))
		{
			OnExplodedViewValueChanged();
		} 
		else if ( PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFractureSettings, FractureLevel))
		{
			OnLevelViewValueChanged();
		}
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFractureSettings, bHideUnselected))
		{
			OnHideUnselectedChanged();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UHistogramSettings, bSorted))
		{
			UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
			HistogramView->RefreshView(HistogramSettings->bSorted);
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UHistogramSettings, InspectedAttribute))
		{
			UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
			HistogramView->InspectAttribute(HistogramSettings->InspectedAttribute);
		}
		else if ( (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UHistogramSettings, bShowRigids)) ||
			(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UHistogramSettings, bShowClusters)) ||
			(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UHistogramSettings, bShowEmbedded)) )
		{
			HistogramView->RegenerateNodes(GetLevelViewValue());
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UOutlinerSettings, ColorByLevel))
		{
			UpdateOutlinerHeader();
			OutlinerView->RegenerateItems();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UOutlinerSettings, ColumnMode))
		{
			UpdateOutlinerHeader();
		}
	}
}

namespace FractureEditorModeToolkit
{
	static const FName GenerateTools = TEXT("Generate");
	static const FName SelectTools = TEXT("Select");
	static const FName FractureTools = TEXT("Fracture");
	static const FName EditTools = TEXT("Edit");
	static const FName ClusterTools = TEXT("Cluster");
	static const FName EmbedTools = TEXT("Embed");
	static const FName UtilitiesTools = TEXT("Utilities");
	static const FName FavoritesTools = TEXT("Favorites");
}

const TArray<FName> FFractureEditorModeToolkit::PaletteNames = 
{ 
	FractureEditorModeToolkit::GenerateTools,
	FractureEditorModeToolkit::SelectTools,
	FractureEditorModeToolkit::FractureTools,
	FractureEditorModeToolkit::EditTools,
	FractureEditorModeToolkit::ClusterTools,
	FractureEditorModeToolkit::EmbedTools,
	FractureEditorModeToolkit::UtilitiesTools,
	FractureEditorModeToolkit::FavoritesTools,
};

FText FFractureEditorModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{ 
	if (Palette == FractureEditorModeToolkit::GenerateTools)
	{
		return LOCTEXT("FractureEditorModeTookit_GenerateTools", "Generate");
	}
	if (Palette == FractureEditorModeToolkit::SelectTools)
	{
		return LOCTEXT("FractureEditorModeTookit_SelectTools", "Select");
	}
	if (Palette == FractureEditorModeToolkit::FractureTools)
	{
		return LOCTEXT("FractureEditorModeTookit_FractureTools", "Fracture");
	}
	if (Palette == FractureEditorModeToolkit::EditTools)
	{
		return LOCTEXT("FractureEditorModeTookit_EditTools", "Edit");
	}
	if (Palette == FractureEditorModeToolkit::ClusterTools)
	{
		return LOCTEXT("FractureEditorModeTookit_ClusterTools", "Cluster");
	}
	if (Palette == FractureEditorModeToolkit::EmbedTools)
	{
		return LOCTEXT("FractureEditorModeTookit_EmbedTools", "Embed");
	}
	if (Palette == FractureEditorModeToolkit::UtilitiesTools)
	{
		return LOCTEXT("FractureEditorModeTookit_UtilitiesTools", "Utilities");
	}
	if (Palette == FractureEditorModeToolkit::FavoritesTools)
	{
		return LOCTEXT("FractureEditorModeTookit_FavoritesTools", "Favorites");
	}
	return FText::FromName(Palette);
}

void FFractureEditorModeToolkit::SetInitialPalette()
{
	// Start in Select Palette if GeometryCollection is selected.
	if (IsGeometryCollectionSelected())
	{
		SetCurrentPalette(FractureEditorModeToolkit::SelectTools);
	}
	else
	{
		SetCurrentPalette(FractureEditorModeToolkit::GenerateTools);
	}
}

void FFractureEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNamesOut) const
{
	PaletteNamesOut = PaletteNames;

	UFractureModeCustomizationSettings* UISettings = GetMutableDefault<UFractureModeCustomizationSettings>();

	// if user has provided custom ordering of tool palettes in the Editor Settings, try to apply them
	if (UISettings->ToolSectionOrder.Num() > 0)
	{
		TArray<FName> NewPaletteNames;
		for (FString SectionName : UISettings->ToolSectionOrder)
		{
			for (int32 k = 0; k < PaletteNamesOut.Num(); ++k)
			{
				if (SectionName.Equals(GetToolPaletteDisplayName(PaletteNamesOut[k]).ToString(), ESearchCase::IgnoreCase)
				 || SectionName.Equals(PaletteNamesOut[k].ToString(), ESearchCase::IgnoreCase))
				{
					NewPaletteNames.Add(PaletteNamesOut[k]);
					PaletteNamesOut.RemoveAt(k);
					break;
				}
			}
		}
		NewPaletteNames.Append(PaletteNamesOut);
		PaletteNamesOut = MoveTemp(NewPaletteNames);
	}

	// if user has provided a list of favorite tools, add that palette to the list
	if (UISettings->ToolFavorites.Num() > 0)
	{
		PaletteNamesOut.Insert(FractureEditorModeToolkit::FavoritesTools, 0);
	}
}

void FFractureEditorModeToolkit::BuildToolPalette(FName PaletteIndex, class FToolBarBuilder& ToolbarBuilder) 
{
	const FFractureEditorCommands& Commands = FFractureEditorCommands::Get();

	if (PaletteIndex == FractureEditorModeToolkit::FavoritesTools)
	{
		UFractureModeCustomizationSettings* UISettings = GetMutableDefault<UFractureModeCustomizationSettings>();

		// build Favorites tool palette
		for (FString ToolName : UISettings->ToolFavorites)
		{
			bool bFound = false;
			TSharedPtr<FUICommandInfo> FoundToolCommand = Commands.FindToolByName(ToolName, bFound);
			if (bFound)
			{
				ToolbarBuilder.AddToolBarButton(FoundToolCommand);
			}
			else
			{
				UE_LOGF(LogFractureTool, Display, "FractureMode: could not find Favorited Tool %ls", *ToolName);
			}
		}
	}
	else if (PaletteIndex == FractureEditorModeToolkit::GenerateTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.GenerateAsset);
		ToolbarBuilder.AddToolBarButton(Commands.ResetAsset);
	}
	else if (PaletteIndex == FractureEditorModeToolkit::SelectTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.SelectAll);
		ToolbarBuilder.AddToolBarButton(Commands.SelectInvert);
		ToolbarBuilder.AddToolBarButton(Commands.SelectNone);
		ToolbarBuilder.AddToolBarButton(Commands.SelectParent);
		ToolbarBuilder.AddToolBarButton(Commands.SelectChildren);
		ToolbarBuilder.AddToolBarButton(Commands.SelectSiblings);
		ToolbarBuilder.AddToolBarButton(Commands.SelectAllInLevel);
		ToolbarBuilder.AddToolBarButton(Commands.SelectNeighbors);
		ToolbarBuilder.AddToolBarButton(Commands.SelectLeaves);
		ToolbarBuilder.AddToolBarButton(Commands.SelectClusters);
		ToolbarBuilder.AddToolBarButton(Commands.SelectCustom);
	}
	else if (PaletteIndex == FractureEditorModeToolkit::FractureTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.Uniform);
		ToolbarBuilder.AddToolBarButton(Commands.Clustered);
		ToolbarBuilder.AddToolBarButton(Commands.Radial);
		ToolbarBuilder.AddToolBarButton(Commands.Planar);
		ToolbarBuilder.AddToolBarButton(Commands.Slice);
		ToolbarBuilder.AddToolBarButton(Commands.Brick);
		ToolbarBuilder.AddToolBarButton(Commands.Mesh);
		ToolbarBuilder.AddToolBarButton(Commands.CustomVoronoi);
	}
	else if (PaletteIndex == FractureEditorModeToolkit::EditTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.DeleteBranch);
		ToolbarBuilder.AddToolBarButton(Commands.Hide);
		ToolbarBuilder.AddToolBarButton(Commands.Unhide);
		ToolbarBuilder.AddToolBarButton(Commands.MergeSelected);
		//ToolbarBuilder.AddToolBarButton(Commands.SplitSelected); // Split tool intentionally disabled; prefer the 'split' option on import instead.
	}
	else if (PaletteIndex == FractureEditorModeToolkit::ClusterTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.AutoCluster);
		ToolbarBuilder.AddToolBarButton(Commands.ClusterMagnet);
		ToolbarBuilder.AddToolBarButton(Commands.Flatten);
		ToolbarBuilder.AddToolBarButton(Commands.Cluster);
		ToolbarBuilder.AddToolBarButton(Commands.Uncluster);
		ToolbarBuilder.AddToolBarButton(Commands.MoveUp);
		ToolbarBuilder.AddToolBarButton(Commands.ClusterMerge);
	}
	else if (PaletteIndex == FractureEditorModeToolkit::EmbedTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.AddEmbeddedGeometry);
		ToolbarBuilder.AddToolBarButton(Commands.AutoEmbedGeometry);
		ToolbarBuilder.AddToolBarButton(Commands.FlushEmbeddedGeometry);
	}
	else if (PaletteIndex == FractureEditorModeToolkit::UtilitiesTools)
	{
		ToolbarBuilder.AddToolBarButton(Commands.AutoUV);
		ToolbarBuilder.AddToolBarButton(Commands.Materials);
		ToolbarBuilder.AddToolBarButton(Commands.RecomputeNormals);
		ToolbarBuilder.AddToolBarButton(Commands.Resample);
		ToolbarBuilder.AddToolBarButton(Commands.ConvertToMesh);
		ToolbarBuilder.AddToolBarButton(Commands.Validate);
		ToolbarBuilder.AddToolBarButton(Commands.MakeConvex);
		ToolbarBuilder.AddToolBarButton(Commands.Proximity);
		ToolbarBuilder.AddToolBarButton(Commands.FixTinyGeo);
		ToolbarBuilder.AddToolBarButton(Commands.SplitIslands);
		ToolbarBuilder.AddToolBarButton(Commands.SetInitialDynamicState);
		ToolbarBuilder.AddToolBarButton(Commands.SetRemoveOnBreak);
	}
}

void FFractureEditorModeToolkit::BindCommands()
{
	const FFractureEditorCommands& Commands = FFractureEditorCommands::Get();
	
	ToolkitCommands->MapAction(
		Commands.ToggleShowBoneColors,
		FExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::ToggleShowBoneColors)//,
	);
	
	ToolkitCommands->MapAction(
		Commands.ViewUpOneLevel,
		FExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::ViewUpOneLevel)//,
	);

	ToolkitCommands->MapAction(
		Commands.ViewDownOneLevel,
		FExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::ViewDownOneLevel)//,
	);

	ToolkitCommands->MapAction(
		Commands.ExplodeMore,
		FExecuteAction::CreateLambda([this]() { this->OnSetExplodedViewValue( FMath::Min(1.0, this->GetExplodedViewValue() + .1) ); } ),
		EUIActionRepeatMode::RepeatEnabled
	);

	ToolkitCommands->MapAction(
		Commands.ExplodeLess,
		FExecuteAction::CreateLambda([this]() { this->OnSetExplodedViewValue( FMath::Max(0.0, this->GetExplodedViewValue() - .1) ); } ),
		EUIActionRepeatMode::RepeatEnabled
	);

	ToolkitCommands->MapAction(
		Commands.CancelTool,
		FExecuteAction::CreateLambda([this]()
		{
			if (GetActiveTool())
			{
				this->SetActiveTool(nullptr);
			}
			else
			{
				GEditor->SelectNone(true, true, false);
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			// don't capture escape when in PIE or simulating
			return GEditor->PlayWorld == NULL && !GEditor->bIsSimulatingInEditor;
		})
	);

	// Map actions of all the Fracture Tools
	TArray<UClass*> SourceClasses = FindFractureToolClasses();
	for (UClass* Class : SourceClasses)
	{
		if (Class->IsChildOf(UFractureModalTool::StaticClass()))
		{
			TSubclassOf<UFractureModalTool> SubclassOf = Class;
			UFractureModalTool* FractureTool = SubclassOf->GetDefaultObject<UFractureModalTool>();

			// Only Bind Commands With Legitimately Set Commands
			if (FractureTool->GetUICommandInfo())
			{
				ToolkitCommands->MapAction(
					FractureTool->GetUICommandInfo(),
					FExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::SetActiveTool, FractureTool),
					FCanExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::CanSetModalTool, FractureTool),
					FIsActionChecked::CreateSP(this, &FFractureEditorModeToolkit::IsActiveTool, FractureTool)
				);
			}
		}
		else
		{
			TSubclassOf<UFractureActionTool> SubclassOf = Class;
			UFractureActionTool* FractureTool = SubclassOf->GetDefaultObject<UFractureActionTool>();

			// Only Bind Commands With Legitimately Set Commands
			if (FractureTool->GetUICommandInfo())
			{
				ToolkitCommands->MapAction(
					FractureTool->GetUICommandInfo(),
					FExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::ExecuteAction, FractureTool),
					FCanExecuteAction::CreateSP(this, &FFractureEditorModeToolkit::CanExecuteAction, FractureTool)
				);
			}
		}

	}
}

void FFractureEditorModeToolkit::SetHideForUnselected(UGeometryCollectionComponent* GCComp)
{
	if (const UGeometryCollection* RestCollection = GCComp->GetRestCollection())
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();

		// If Hide managed array exists, set false for any selected bones, true for selected. 
		// If a cluster is selected, set false for all children.
		if (GeometryCollection->HasAttribute("Hide", FGeometryCollection::TransformGroup))
		{
			TManagedArray<bool>& Hide = GeometryCollection->ModifyAttribute<bool>("Hide", FGeometryCollection::TransformGroup);
			const TManagedArray<TSet<int32>>& Children = GeometryCollection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
			
			const TArray<int32>& SelectedBones = GCComp->GetSelectedBones();
			if (SelectedBones.Num() > 0)
			{ 
				Hide.Fill(true);

				for (int32 SelectedBone : SelectedBones)
				{
					if (!ensure(SelectedBone >= 0 && SelectedBone < Hide.Num()))
					{
						// Invalid selection, don't hide anything
						Hide.Fill(false);
						break;
					}
					Hide[SelectedBone] = false;
					if (Children[SelectedBone].Num() > 0)
					{ 
						TArray<int32> BranchBones;
						FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(Children, SelectedBone, BranchBones);
						for (int32 BranchBone : BranchBones)
						{
							Hide[BranchBone] = false;
						}
					}
				}
			}
			else
			{
				// Don't hide anything if we've selected nothing
				Hide.Fill(false);
			}

			GCComp->RefreshEmbeddedGeometry();

		}
	}
}

void FFractureEditorModeToolkit::HandleMapChanged(class UWorld* NewWorld, EMapChangeType MapChangeType)
{
	if ((MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap || MapChangeType == EMapChangeType::TearDownWorld))
	{
		ShutdownActiveTool();
		TArray<UGeometryCollectionComponent*> EmptySelection;
		SetOutlinerComponents(EmptySelection);
	}
}

void FFractureEditorModeToolkit::OnToolPaletteChanged(FName PaletteName) 
{
	if (GetActiveTool() != nullptr)
	{
		SetActiveTool(0);
	}
}

FName FFractureEditorModeToolkit::GetToolkitFName() const
{
	return FName("FractureEditorMode");
}

FText FFractureEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("FractureEditorModeToolkit", "DisplayName", "FractureEditorMode Tool");
}

class FEdMode* FFractureEditorModeToolkit::GetEditorMode() const
{
	return GLevelEditorModeTools().GetActiveMode(UFractureEditorMode::EM_FractureEditorModeId);
}

void FFractureEditorModeToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ActiveTool);
}

float FFractureEditorModeToolkit::GetExplodedViewValue() const
{
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	return FractureSettings->ExplodeAmount;
}

int32 FFractureEditorModeToolkit::GetLevelViewValue() const
{
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	return FractureSettings->FractureLevel;
}

bool FFractureEditorModeToolkit::GetHideUnselectedValue() const
{
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	return FractureSettings->bHideUnselected;
}

void FFractureEditorModeToolkit::OnSetExplodedViewValue(float NewValue)
{
	FScopedTransaction Transaction(LOCTEXT("SetExplodedViewValue", "Adjust Exploded View"));

	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	if ( FMath::Abs<float>( FractureSettings->ExplodeAmount - NewValue ) >= .01f)
	{
		FractureSettings->ExplodeAmount = NewValue;
		OnExplodedViewValueChanged();
	}
}

void FFractureEditorModeToolkit::OnExplodedViewValueChanged()
{
	USelection* SelectionSet = GEditor->GetSelectedActors();

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());
	SelectionSet->GetSelectedObjects(SelectedActors);

	for (AActor* Actor : SelectedActors)
	{
		TInlineComponentArray<UPrimitiveComponent*> Components;
		Actor->GetComponents(Components);
		for (UPrimitiveComponent* PrimitiveComponent : Components)
		{
			if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(PrimitiveComponent))
			{

				UpdateExplodedVectors(GeometryCollectionComponent);

				GeometryCollectionComponent->MarkRenderStateDirty();
			}
		}
	}

	GCurrentLevelEditingViewportClient->Invalidate();
}


int32 FFractureEditorModeToolkit::GetLevelCount()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	int32 ReturnLevel = -1;
	for (UGeometryCollectionComponent* Comp : GeomCompSelection)
	{
		FGeometryCollectionEdit GCEdit = Comp->EditRestCollection(GeometryCollection::EEditUpdate::None);
		if (UGeometryCollection* GCObject = GCEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GCObject->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				bool HasLevelAttribute = GeometryCollection->HasAttribute("Level", FTransformCollection::TransformGroup);
				if (HasLevelAttribute)
				{
					const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);

					if(Levels.Num() > 0)
					{
						for (int32 Level : Levels)
						{
							if (Level > ReturnLevel)
							{
								ReturnLevel = Level;
							}
						}
					}
				}
			}
		}
	}
	return ReturnLevel + 1;
}


void FFractureEditorModeToolkit::OnSetLevelViewValue(int32 NewValue)
{
	FScopedTransaction Transaction(LOCTEXT("SetLevelViewValue", "Adjust View Level"));

	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	FractureSettings->FractureLevel = NewValue;
	OnLevelViewValueChanged();
}

void FFractureEditorModeToolkit::OnLevelViewValueChanged()
{
	int32 FractureLevel = GetLevelViewValue();

	USelection* SelectionSet = GEditor->GetSelectedActors();

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());
	SelectionSet->GetSelectedObjects(SelectedActors);

	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* Comp : GeomCompSelection)
	{
		FScopedColorEdit EditBoneColor = Comp->EditBoneSelection();
		if(EditBoneColor.GetViewLevel() != FractureLevel)
		{
			EditBoneColor.SetLevelViewMode(FractureLevel);
			// Clear selection below currently-selected view level and update highlights,
			// so the selection is compatible with the current 3D view and outliner (e.g., doesn't hide selection of children)
			EditBoneColor.Sanitize();
			EditBoneColor.FilterSelectionToLevel();
			UpdateExplodedVectors(Comp);
			Comp->MarkRenderStateDirty();
			Comp->MarkRenderDynamicDataDirty();
		}
	}
	SetOutlinerComponents(GeomCompSelection.Array());

	GCurrentLevelEditingViewportClient->Invalidate();
}

void FFractureEditorModeToolkit::UpdateHideForComponent(UGeometryCollectionComponent* Comp)
{
	if (const UGeometryCollection* RestCollection = Comp->GetRestCollection())
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();

		if (GetHideUnselectedValue())
		{
			// If we are toggling on, add and configure the Hide array.
			if (!GeometryCollection->HasAttribute("Hide", FGeometryCollection::TransformGroup))
			{
				GeometryCollection->AddAttribute<bool>("Hide", FGeometryCollection::TransformGroup);
			}
			SetHideForUnselected(Comp);
		}
		else
		{
			// If we are toggling off, remove the Hide array.
			if (GeometryCollection->HasAttribute("Hide", FGeometryCollection::TransformGroup))
			{
				GeometryCollection->RemoveAttribute("Hide", FGeometryCollection::TransformGroup);
			}
			Comp->RefreshEmbeddedGeometry();
		}
	}
}

void FFractureEditorModeToolkit::OnHideUnselectedChanged()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* Comp : GeomCompSelection)
	{
		if (const UGeometryCollection* RestCollection = Comp->GetRestCollection())
		{ 
			UpdateHideForComponent(Comp);
		
			// redraw
			Comp->MarkRenderDynamicDataDirty();
			Comp->MarkRenderStateDirty();
		}
	}
}

void FFractureEditorModeToolkit::ToggleShowBoneColors()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* Comp : GeomCompSelection)
	{
		FScopedColorEdit EditBoneColor(Comp, true /*bForceUpdate*/); // the property has already changed; this will trigger the color update + render state updates
		EditBoneColor.SetShowBoneColors(!EditBoneColor.GetShowBoneColors());
	}
}

void FFractureEditorModeToolkit::ViewUpOneLevel()
{
	int32 CountMax = GetLevelCount() + 1;
	int32 NewLevel = ((GetLevelViewValue() + CountMax) % CountMax) - 1;
	OnSetLevelViewValue(NewLevel);
}

void FFractureEditorModeToolkit::ViewDownOneLevel()
{
	int32 CountMax = GetLevelCount() + 1;
	int32 NewLevel = ((GetLevelViewValue() + CountMax + 2 ) % CountMax) - 1;
	OnSetLevelViewValue(NewLevel);
}

TSharedRef<SWidget> FFractureEditorModeToolkit::GetLevelViewMenuContent(TSharedRef<IPropertyHandle> PropertyHandle)
{
	int32 FractureLevel = GetLevelViewValue();

	FMenuBuilder MenuBuilder(true, GetToolkitCommands());

	MenuBuilder.AddMenuEntry(
		LOCTEXT("LevelMenuAll", "All Levels"),
		LOCTEXT("LevelMenuAllTooltip", "View All Leaf Bones in this Geometry Collection"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([=] { PropertyHandle->SetValue(-1); } ),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([=] {return FractureLevel == -1 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;})
		)
	);

	MenuBuilder.AddMenuSeparator();

	for (int32 i = 0; i < GetLevelCount(); i++)
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("LevelMenuN", "Level {0}"), FText::AsNumber(i)),
			FText::Format(LOCTEXT("LevelMenuNTooltip", "View Level {0} in this Geometry Collection"), FText::AsNumber(i)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=] { PropertyHandle->SetValue(i); } ),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([=] {return FractureLevel == i ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;})
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FFractureEditorModeToolkit::GetViewMenuContent()
{
	const FFractureEditorCommands& Commands = FFractureEditorCommands::Get();

	FMenuBuilder MenuBuilder(false, GetToolkitCommands());
 	MenuBuilder.AddMenuEntry(Commands.ToggleShowBoneColors);

	return MenuBuilder.MakeWidget();
}

void FFractureEditorModeToolkit::ExecuteAction(UFractureActionTool* InActionTool)
{
	if (InActionTool)
	{
		InActionTool->Execute(StaticCastSharedRef<FFractureEditorModeToolkit>(AsShared()));

		InvalidateHitProxies();
	}
}

void FFractureEditorModeToolkit::InvalidateHitProxies()
{
	if (GIsEditor)
	{
		for (FEditorViewportClient* Viewport : GEditor->GetLevelViewportClients())
		{
			Viewport->Invalidate();
		}
	}
}

bool FFractureEditorModeToolkit::CanExecuteAction(UFractureActionTool* InActionTool) const
{
	// Disallow fracture actions when playing in editor or simulating.
	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		return false;
	}

	if (InActionTool)
	{
		return InActionTool->CanExecute();
	}
	else
	{
		return false;
	}
}

void FFractureEditorModeToolkit::ShutdownActiveTool()
{
	if (ActiveTool)
	{
		ActiveTool->Shutdown();
		ActiveTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);

		ActiveTool = nullptr;
	}
}

bool FFractureEditorModeToolkit::CanSetModalTool(UFractureModalTool* InActiveTool) const
{
	// Disallow fracture modal tools when playing in editor or simulating.
	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		return false;
	}

	return true;
}

void FFractureEditorModeToolkit::SetActiveTool(UFractureModalTool* InActiveTool)
{
	ShutdownActiveTool();

	ActiveTool = InActiveTool;

	TArray<UObject*> ToolSettings;
	if (ActiveTool != nullptr)
	{
		ActiveTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FFractureEditorModeToolkit::InvalidateCachedDetailPanelState);

		ActiveTool->Setup(StaticCastSharedRef<FFractureEditorModeToolkit>(AsShared()));

		ToolSettings.Append(ActiveTool->GetSettingsObjects());

		ActiveTool->SelectedBonesChanged();
		ActiveTool->FractureContextChanged();
	}

	DetailsView->SetObjects(ToolSettings);
}

void FFractureEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	DetailsView->InvalidateCachedState();
}

void FFractureEditorModeToolkit::Shutdown()
{
	ShutdownActiveTool();

	FEditorDelegates::BeginPIE.Remove(BeginPIEDelegateHandle);
}


UFractureModalTool* FFractureEditorModeToolkit::GetActiveTool() const
{
	return ActiveTool;
}

bool FFractureEditorModeToolkit::IsActiveTool(UFractureModalTool* InActiveTool)
{
	return bool(ActiveTool == InActiveTool);
}

FText FFractureEditorModeToolkit::GetActiveToolDisplayName() const
{
	if (ActiveTool != nullptr)
	{
		return ActiveTool->GetDisplayText();	
	}
	return LOCTEXT("FractureNoTool", "Fracture Editor");
}

FText FFractureEditorModeToolkit::GetActiveToolMessage() const
{
	if (ActiveTool != nullptr)
	{
		return ActiveTool->GetTooltipText();	
	}
	return LOCTEXT("FractureNoToolMessage", "Select geometry and use “New+” to create a new Geometry Collection to begin fracturing.  Choose one of the fracture tools to break apart the selected Geometry Collection.");
}

namespace UE::PrivateOutlinerCacheHelper
{
	void AddGeometryStatsForComponent(const UGeometryCollectionComponent* Component, int64& BoneCount, int64& VertexCount, int64& HullCount)
	{
		if (Component)
		{
			if (const UGeometryCollection* GeometryObject = Component->GetRestCollection())
			{
				if (const FGeometryCollection* Collection = GeometryObject->GetGeometryCollection().Get())
				{
					BoneCount += Collection->Transform.Num();
					VertexCount += Collection->Vertex.Num();
					HullCount += Collection->NumElements(FGeometryCollection::ConvexGroup);
				}
			}
		}
	}

	void ComputeGeometryStats(const TArray<UGeometryCollectionComponent*>& InNewComponents, int64& BoneCount, int64& VertexCount, int64& HullCount)
	{
		BoneCount = VertexCount = HullCount = 0;
		for (const UGeometryCollectionComponent* Component : InNewComponents)
		{
			AddGeometryStatsForComponent(Component, BoneCount, VertexCount, HullCount);
		}
	}

	void ComputeGeometryStats(const TArray<TWeakObjectPtr<UGeometryCollectionComponent>>& InNewComponents, int64& BoneCount, int64& VertexCount, int64& HullCount)
	{
		BoneCount = VertexCount = HullCount = 0;
		for (const TWeakObjectPtr<UGeometryCollectionComponent> Component : InNewComponents)
		{
			AddGeometryStatsForComponent(Component.Get(), BoneCount, VertexCount, HullCount);
		}
	}
}

bool FFractureEditorModeToolkit::IsCachedOutlinerGeometryStale(const TArray<TWeakObjectPtr<UGeometryCollectionComponent>>& SelectedComponents) const
{
	// Note we currently use quick-to-compute high level stats, compared vs cached versions, since this is run per tick; we could change this to more thoroughly walk the outliner data potentially
	int64 NewBoneCount, NewVertexCount, NewHullCount;
	UE::PrivateOutlinerCacheHelper::ComputeGeometryStats(SelectedComponents, NewBoneCount, NewVertexCount, NewHullCount);
	return NewBoneCount != OutlinerCachedBoneCount || NewVertexCount != OutlinerCachedVertexCount || NewHullCount != OutlinerCachedHullCount;
}

void FFractureEditorModeToolkit::SetOutlinerComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents)
{
	// Update cached stats (bone and vertex count) of the components in the outliner
	UE::PrivateOutlinerCacheHelper::ComputeGeometryStats(InNewComponents, OutlinerCachedBoneCount, OutlinerCachedVertexCount, OutlinerCachedHullCount);

	TArray<UGeometryCollectionComponent*> ComponentsToEdit;
	ComponentsToEdit.Reserve(InNewComponents.Num());
	for (UGeometryCollectionComponent* Component : InNewComponents)
	{
		if (!Component)
		{
			continue;
		}
		FGeometryCollectionEdit RestCollection = Component->EditRestCollection(GeometryCollection::EEditUpdate::None);
		UGeometryCollection* FracturedGeometryCollection = RestCollection.GetRestCollection();

		if (IsValid(FracturedGeometryCollection)) // Prevents crash when GC is deleted from content browser and actor is selected.
		{
			if (TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = FracturedGeometryCollection->GetGeometryCollection())
			{
				FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollectionPtr.Get(), -1);
				UpdateExplodedVectors(Component);
				UpdateHideForComponent(Component);

				UpdateGeometryComponentAttributes(Component);
				ComponentsToEdit.Add(Component);

				Component->MarkRenderStateDirty();
			}
		}	
	}

	if (OutlinerView)
	{
		OutlinerView->SetComponents(ComponentsToEdit);
	}

	if (HistogramView)
	{
		HistogramView->SetComponents(ComponentsToEdit, GetLevelViewValue());
	}

	if (StatisticsView)
	{
		FGeometryCollectionStatistics Stats;
		GetStatisticsSummary(Stats);
		StatisticsView->SetStatistics(Stats);
	}

	// update view rest collection
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	FractureSettings->RestCollection = nullptr;
	if (InNewComponents.Num() > 0)
	{
		UGeometryCollectionComponent* Component = InNewComponents[0];
		FractureSettings->RestCollection = Component->GetRestCollection();
	}

	RefreshVariableOverrideDetailView(FractureSettings->RestCollection.Get());

	if (ActiveTool != nullptr)
	{
		ActiveTool->SelectedBonesChanged();
		ActiveTool->FractureContextChanged();
	}
}

void FFractureEditorModeToolkit::SetBoneSelection(UGeometryCollectionComponent* InRootComponent, const TArray<int32>& InSelectedBones, bool bClearCurrentSelection, int32 FocusBoneIdx)
{
	OutlinerView->SetBoneSelection(InRootComponent, InSelectedBones, bClearCurrentSelection, FocusBoneIdx);
	HistogramView->SetBoneSelection(InRootComponent, InSelectedBones, bClearCurrentSelection, FocusBoneIdx);

	UpdateHideForComponent(InRootComponent);
	
	if (ActiveTool != nullptr)
	{
		ActiveTool->SelectedBonesChanged();
		ActiveTool->FractureContextChanged();
	}
}

FReply FFractureEditorModeToolkit::OnModalClicked()
{
	if (ActiveTool)
	{
		const double CacheStartTime = FPlatformTime::Seconds();

		FScopedTransaction Transaction(LOCTEXT("FractureMesh", "Fracture Mesh"));

		ActiveTool->Execute(StaticCastSharedRef<FFractureEditorModeToolkit>(AsShared()));

		float ProcessingTime = static_cast<float>(FPlatformTime::Seconds() - CacheStartTime);

		GCurrentLevelEditingViewportClient->Invalidate();

	}

	return FReply::Handled();
}

bool FFractureEditorModeToolkit::CanExecuteModal() const
{
	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		return false;
	}

	if (!IsSelectedActorsInEditorWorld())
	{
		return false;
	}

	if (ActiveTool != nullptr) 
	{
		return ActiveTool->CanExecute();
	}
	
	return false;
}

void FFractureEditorModeToolkit::GetSelectedGeometryCollectionComponents(TSet<UGeometryCollectionComponent*>& GeomCompSelection)
{
	USelection* SelectionSet = GEditor->GetSelectedActors();
	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());
	SelectionSet->GetSelectedObjects(SelectedActors);

	GeomCompSelection.Empty(SelectionSet->Num());

	for (AActor* Actor : SelectedActors)
	{
		TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
		Actor->GetComponents(GeometryCollectionComponents);
		GeomCompSelection.Append(GeometryCollectionComponents);
	}
}

void FFractureEditorModeToolkit::AddAdditionalAttributesIfRequired(UGeometryCollection* GeometryCollectionObject)
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
	if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
	{
		if (!GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
		{
			FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
		}
	}
}

bool FFractureEditorModeToolkit::IsGeometryCollectionSelected()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<ULevel*> UniqueLevels;
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			if (Actor->FindComponentByClass<UGeometryCollectionComponent>())
			{
				return true;
			}
		}
	}
	return false;
}

bool FFractureEditorModeToolkit::IsSelectedActorsInEditorWorld()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			check(Actor->GetWorld());
			if (Actor->GetWorld()->WorldType != EWorldType::Editor)
			{
				return false;
			}
		}
	}
	return true;
}


void FFractureEditorModeToolkit::UpdateGeometryComponentAttributes(UGeometryCollectionComponent* Component)
{
	if (Component)
	{
		const UGeometryCollection* RestCollection = Component->GetRestCollection();
		if (RestCollection && IsValidChecked(RestCollection))
		{
			FGeometryCollectionPtr GeometryCollection = RestCollection->GetGeometryCollection();
			if (!GeometryCollection->HasAttribute("Volume", FTransformCollection::TransformGroup))
			{
				// Note: SetVolumeAttributes (below) will add the attribute as needed
				UE_LOGF(LogFractureTool, Warning, "Added Volume attribute to GeometryCollection.");
			}

			// TODO: this should instead be called systematically in FGeometryCollectionEdit or similar
			// (it is currently also called by the convex generation, however it is relatively fast so is ok if we call it twice)
			FGeometryCollectionConvexUtility::SetVolumeAttributes(GeometryCollection.Get());
		}
	}
	
}


bool GetValidGeoCenter(FGeometryCollection* Collection, const TManagedArray<int32>& TransformToGeometryIndex, const TArray<FTransform>& Transforms, const TManagedArray<TSet<int32>>& Children, const TManagedArray<FBox>& BoundingBox, int32 TransformIndex, FVector& OutGeoCenter )
{
	if (Collection->IsRigid(TransformIndex))
	{
		OutGeoCenter = Transforms[TransformIndex].TransformPosition(BoundingBox[TransformToGeometryIndex[TransformIndex]].GetCenter());

		return true;
	}
	else if (Collection->SimulationType[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_None) // ie this is embedded geometry
	{
		int32 Parent = Collection->Parent[TransformIndex];
		int32 ParentGeo = Parent != INDEX_NONE ? TransformToGeometryIndex[Parent] : INDEX_NONE;
		if (ensureMsgf(ParentGeo != INDEX_NONE, TEXT("Embedded geometry should always have a rigid geometry parent!  Geometry collection may be malformed.")))
		{
			OutGeoCenter = Transforms[Collection->Parent[TransformIndex]].TransformPosition(BoundingBox[ParentGeo].GetCenter());
		}
		else
		{
			return false; // no valid value to return
		}

		return true;
	}
	else
	{
		FVector AverageCenter;
		int32 ValidVectors = 0;
		for(int32 ChildIndex : Children[TransformIndex])
		{

			if (GetValidGeoCenter(Collection, TransformToGeometryIndex, Transforms, Children, BoundingBox, ChildIndex, OutGeoCenter))
			{
				if (ValidVectors == 0)
				{
					AverageCenter = OutGeoCenter;
				}
				else
				{
					AverageCenter += OutGeoCenter;
				}
				++ValidVectors;
			}
		}

		if (ValidVectors > 0)
		{
			OutGeoCenter = AverageCenter / ValidVectors;
			return true;
		}
	}
	return false;
}

void FFractureEditorModeToolkit::UpdateExplodedVectors(UGeometryCollectionComponent* GeometryCollectionComponent) const
{
#if WITH_EDITOR
	// If we're running PIE or SIE when this happens we should ignore the rebuild as the implicits will be in use.
	if(GEditor->bIsSimulatingInEditor || GEditor->GetPIEWorldContext() != nullptr)
	{
		return;
	}
#endif

	if (GeometryCollectionComponent == nullptr || GeometryCollectionComponent->GetRestCollection() == nullptr)
	{
		return;
	}

	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection();
	const FGeometryCollection* OutGeometryCollectionConst = GeometryCollectionPtr.Get();

	float ExplodeAmount = GetExplodedViewValue();

	if (FMath::IsNearlyEqual(ExplodeAmount, 0.0f))
	{
		if (OutGeometryCollectionConst->HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup))
		{
			// ExplodedVector is not saved, so the Rest collection doesn't 'see' this update in serialization, so we don't need EEditUpdate::Rest here
			FGeometryCollectionEdit RestCollection = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::Dynamic);
			FGeometryCollection* OutGeometryCollection = GeometryCollectionPtr.Get();
			OutGeometryCollection->RemoveAttribute("ExplodedVector", FGeometryCollection::TransformGroup);
		}
	}
	else
	{
		// ExplodedVector is not saved, so the Rest collection doesn't 'see' this update in serialization, so we don't need EEditUpdate::Rest here
		FGeometryCollectionEdit RestCollection = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::Dynamic);
		UGeometryCollection* GeometryCollection = RestCollection.GetRestCollection();
		FGeometryCollection* OutGeometryCollection = GeometryCollectionPtr.Get();

		if (!OutGeometryCollection->HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup))
		{
			OutGeometryCollection->AddAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup, FManagedArrayCollection::FConstructionParameters(FName(), false));
		}

		check(OutGeometryCollection->HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup));

		TManagedArray<FVector3f>& ExplodedVectors = OutGeometryCollection->ModifyAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);
		const TManagedArray<FTransform3f>& Transform = OutGeometryCollection->GetAttribute<FTransform3f>("Transform", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& TransformToGeometryIndex = OutGeometryCollection->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
		const TManagedArray<FBox>& BoundingBox = OutGeometryCollection->GetAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);

		// Make sure we have valid "Level"
		AddAdditionalAttributesIfRequired(GeometryCollection);

		const TManagedArray<int32>& Levels = OutGeometryCollection->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);
		const TManagedArray<int32>& Parent = OutGeometryCollection->GetAttribute<int32>("Parent", FTransformCollection::TransformGroup);
		const TManagedArray<TSet<int32>>& Children = OutGeometryCollection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);

		int32 ViewFractureLevel = GetLevelViewValue();

		int32 MaxFractureLevel = ViewFractureLevel;
		for (int32 Idx = 0, ni = GeometryCollection->NumElements(FGeometryCollection::TransformGroup); Idx < ni; ++Idx)
		{
			if (Levels[Idx] > MaxFractureLevel)
				MaxFractureLevel = Levels[Idx];
		}

		TArray<FTransform> Transforms;
		GeometryCollectionAlgo::GlobalMatrices(Transform, OutGeometryCollection->Parent, Transforms);

		TArray<FVector> TransformedCenters;
		TransformedCenters.SetNumUninitialized(Transforms.Num());

		int32 TransformsCount = 0;

		FVector Center(ForceInitToZero);
		for (int32 Idx = 0, ni = GeometryCollection->NumElements(FGeometryCollection::TransformGroup); Idx < ni; ++Idx)
		{
			ExplodedVectors[Idx] = FVector3f::ZeroVector;
			FVector GeoCenter;
			if (GetValidGeoCenter(GeometryCollection->GetGeometryCollection().Get(), TransformToGeometryIndex, Transforms, Children, BoundingBox, Idx, GeoCenter))
			{
				TransformedCenters[Idx] = GeoCenter;
				if ((ViewFractureLevel < 0) || Levels[Idx] == ViewFractureLevel)
				{
					Center += TransformedCenters[Idx];
					++TransformsCount;
				}
			}
		}

		Center /= TransformsCount;

		for (int Level = 1; Level <= MaxFractureLevel; Level++)
		{
			for (int32 Idx = 0, ni = GeometryCollection->NumElements(FGeometryCollection::TransformGroup); Idx < ni; ++Idx)
			{
				if ((ViewFractureLevel < 0) || Levels[Idx] == ViewFractureLevel)
				{
					ExplodedVectors[Idx] = (FVector3f)(TransformedCenters[Idx] - Center) * ExplodeAmount;
				}
				else
				{
					if (Parent[Idx] > -1)
					{
						ExplodedVectors[Idx] = ExplodedVectors[Parent[Idx]];
					}
				}
			}
		}
	}

	GeometryCollectionComponent->RefreshEmbeddedGeometry();
	GeometryCollectionComponent->UpdateCachedBounds();
}

void FFractureEditorModeToolkit::RegenerateOutliner()
{
	OutlinerView->UpdateGeometryCollection();
}

void FFractureEditorModeToolkit::RegenerateHistogram()
{
	HistogramView->RegenerateNodes(GetLevelViewValue());
}

void FFractureEditorModeToolkit::OnOutlinerBoneSelectionChanged(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones)
{
	const UGeometryCollection* RestCollection = RootComponent->GetRestCollection();
	if (RestCollection && IsValidChecked(RestCollection))
	{
		int32 NumValidBones = Algo::RemoveIf(SelectedBones, [&RestCollection](const int32& Bone)
			{
				return Bone < 0 || Bone >= RestCollection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup);
			});
		if (!ensure(NumValidBones == SelectedBones.Num())) // Protect against invalid bones in selection, but ensure() as this indicates the UI is out of sync with the data
		{
			SelectedBones.SetNum(NumValidBones);
		}
		if (SelectedBones.Num())
		{
			// don't need to snap the bones to the current level because they are directly selected from the outliner
			FFractureSelectionTools::ToggleSelectedBones(RootComponent, SelectedBones, true, false, false /*bSnapToLevel*/);
			OutlinerView->SetBoneSelection(RootComponent, SelectedBones, true);
			HistogramView->SetBoneSelection(RootComponent, SelectedBones, true);
		}
		else
		{
			FFractureSelectionTools::ClearSelectedBones(RootComponent);
		}

		if (ActiveTool != nullptr)
		{
			ActiveTool->SelectedBonesChanged();
			ActiveTool->FractureContextChanged();
		}

		UpdateHideForComponent(RootComponent);

		RootComponent->MarkRenderStateDirty();
		RootComponent->MarkRenderDynamicDataDirty();
	}
}

void FFractureEditorModeToolkit::OnHistogramBoneSelectionChanged(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones)
{
	const UGeometryCollection* RestCollection = RootComponent->GetRestCollection();
	if (RestCollection && IsValidChecked(RestCollection))
	{
		if (SelectedBones.Num())
		{

			FFractureSelectionTools::ToggleSelectedBones(RootComponent, SelectedBones, true, false);
			OutlinerView->SetBoneSelection(RootComponent, SelectedBones, true);
			HistogramView->SetBoneSelection(RootComponent, SelectedBones, true);
		}
		else
		{
			FFractureSelectionTools::ClearSelectedBones(RootComponent);
		}

		if (ActiveTool != nullptr)
		{
			ActiveTool->SelectedBonesChanged();
			ActiveTool->FractureContextChanged();
		}

		UpdateHideForComponent(RootComponent);

		RootComponent->MarkRenderStateDirty();
		RootComponent->MarkRenderDynamicDataDirty();
	}

}

FText FFractureEditorModeToolkit::GetSelectionInfo() const
{
	return FText::Format(LOCTEXT("FractureEditorModeToolkit_SelectionInfo", "Selected: {0} / {1}"), OutlinerView->GetBoneSelectionCount(), OutlinerView->GetBoneDisplayedCount());
}


void FFractureEditorModeToolkit::GetStatisticsSummary(FGeometryCollectionStatistics& Stats) const
{
	TArray<const FGeometryCollection*> GeometryCollectionArray;
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AActor* Actor = Cast<AActor>(*Iter))
			{
				TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
				Actor->GetComponents(GeometryCollectionComponents);
				for (UGeometryCollectionComponent* GeometryCollectionComponent : GeometryCollectionComponents)
				{
					if (const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection())
					{
						if (RestCollection->GetGeometryCollection())
						{
							GeometryCollectionArray.Add(RestCollection->GetGeometryCollection().Get());
						}
					}
				}
			}
		}
	}

	if (GeometryCollectionArray.Num() > 0)
	{
		TArray<uint32> TransformCountPerLevel;
		TArray<uint32> ConvexCountPerLevel;
		int32 LevelMax = INT_MIN;
		int32 EmbeddedCount = 0;

		for (int32 Idx = 0; Idx < GeometryCollectionArray.Num(); ++Idx)
		{
			const FGeometryCollection* GeometryCollection = GeometryCollectionArray[Idx];

			check(GeometryCollection);

			if(const TManagedArray<int32>* Levels = GeometryCollection->FindAttribute<int32>("Level", FGeometryCollection::TransformGroup))
			{

				// num transforms per level
				const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

				TArray<int32> LevelTransforms;
				for(int32 Element = 0, NumElement = Levels->Num(); Element < NumElement; ++Element)
				{
					if (SimulationType[Element] == FGeometryCollection::ESimulationTypes::FST_None)
					{
						++EmbeddedCount;
					}
					else
					{
						const int32 NodeLevel = (*Levels)[Element];
						if (LevelTransforms.Num() <= NodeLevel)
						{
							LevelTransforms.SetNumZeroed(NodeLevel + 1);
						}
						++LevelTransforms[NodeLevel];
					}		
				}

				if (TransformCountPerLevel.Num() < LevelTransforms.Num())
				{
					TransformCountPerLevel.SetNumZeroed(LevelTransforms.Num());
				}
				for(int32 Level = 0; Level < LevelTransforms.Num(); ++Level)
				{
					TransformCountPerLevel[Level] += LevelTransforms[Level];
				}

				if(LevelTransforms.Num() > LevelMax)
				{
					LevelMax = LevelTransforms.Num();
				}

				// convex per level
				// todo(chaos) : should probably share some of this logic with the outliner 
				TManagedArrayAccessor<TSet<int32>> GCTransformToConvexIndicesAttribute(*GeometryCollection, "TransformToConvexIndices", FGeometryCollection::TransformGroup);
				if (GCTransformToConvexIndicesAttribute.IsValid())
				{
					const TManagedArray<TSet<int32>>& GCTransformToConvexIndices = GCTransformToConvexIndicesAttribute.Get();

					Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeometryCollection);
					const TArray<int32> TransformIndices = HierarchyFacade.GetTransformArrayInDepthFirstOrder();

					TArray<int32> ConvexCountArray;
					ConvexCountArray.SetNumZeroed(TransformIndices.Num());

					int32 MaxLevel = 0;
					for (int32 TransformIndex : TransformIndices)
					{
						const int32 ConvexCount = GCTransformToConvexIndices[TransformIndex].Num();
						if (ConvexCount > 0)
						{
							ConvexCountArray[TransformIndex] = ConvexCount;
						}
						// if count == 0 then we have already accumulated the children in the parents, no need to do anything
						// so now just pass it to the direct parent ( we parse the index in a depth first manner ) 
						const int32 ParentTransformIndex = GeometryCollection->Parent[TransformIndex];
						if (ParentTransformIndex != INDEX_NONE)
						{
							// if parent has no convex then it will be a union of the aggregated children
							const int32 ParentConvexCount = GCTransformToConvexIndices[ParentTransformIndex].Num();
							if (ParentConvexCount == 0)
							{
								ConvexCountArray[ParentTransformIndex] += ConvexCountArray[TransformIndex];
							}
						}

						const int32 Level = (*Levels)[TransformIndex];
						MaxLevel = FMath::Max(MaxLevel, Level);
					}
					ConvexCountPerLevel.SetNumZeroed(MaxLevel + 1);
					for (int32 TransformIndex = 0; TransformIndex < ConvexCountArray.Num(); TransformIndex++)
					{
						const int32 Level = (*Levels)[TransformIndex];
						ConvexCountPerLevel[Level] += ConvexCountArray[TransformIndex];
					}
				}
			}
		}

		Stats.CountsPerLevel = TransformCountPerLevel;
		Stats.ConvexCountPerLevel = ConvexCountPerLevel;
		Stats.EmbeddedCount = EmbeddedCount;
	}
}

void FFractureEditorModeToolkit::CreateDataflowProgressBar()
{
	const FSlateBrush* OverlayBrush = FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush");

	auto GetBarVisibility = [this]() ->EVisibility
		{
			const bool bVisible = DataflowEvaluator ? DataflowEvaluator->IsRunning() : false;
			return bVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
		};

	auto GetProgressText = [this]() -> FText
		{
			if (DataflowEvaluator)
			{
				return FText::Format(
					LOCTEXT("DataflowGraphEditor_ProgressText", "Total Progress : {0}% - {1}")
					, DataflowEvaluator->GetProgress() * 100.f
					, DataflowEvaluator->GetProgressMessage()
				);
			}
			return FText::GetEmpty();
		};

	auto GetProgressPercent = [this]() -> float
		{
			return DataflowEvaluator ? DataflowEvaluator->GetProgress() : 0.f;
		};

	auto OnCancel = [this]() -> FReply
		{
			if (DataflowEvaluator)
			{
				DataflowEvaluator->Cancel();
			}
			return FReply::Handled();
		};

	SAssignNew(DataflowProgressBar, SHorizontalBox)
		.Visibility_Lambda(GetBarVisibility)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Bottom)
		.FillWidth(1.0)
		.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
		[
			SNew(SBorder)
			.BorderImage(OverlayBrush)
			[
				SNew(SVerticalBox)

				// Text 
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4)
				.VAlign(VAlign_Center)
				[
					// Progress text
					SNew(STextBlock)
					.Text_Lambda(GetProgressText)
				]

				// Progress Bar + Cancel button
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					// Progress bar 
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.Padding(4)
					[
						SNew(SProgressBar)
						.Percent_Lambda(GetProgressPercent)
					]

					// Cancel button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4)
					[
						SNew(SButton)
						.Text(LOCTEXT("DataflowGraphEditor_CancelText", "Cancel"))
						.ToolTipText(LOCTEXT("DataflowGraphEditor_CancelTooltip", "Cancel current evaluation"))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked_Lambda(OnCancel)
					]
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
