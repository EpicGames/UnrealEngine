// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureEditorStyle.h"

#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"

FName FFractureEditorStyle::StyleName("FractureEditorStyle");

FFractureEditorStyle::FFractureEditorStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D IconSize(20.0f, 20.0f);
	const FVector2D LabelIconSize(16.0f, 16.0f);


	// const FVector2D Icon8x8(8.0f, 8.0f);
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ChaosEditor/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("FractureEditor.DefaultSettings", new FSlateImageBrush(RootToCoreContentDir(TEXT("../Editor/Slate/Icons/GeneralTools/Settings_40x.png")), IconSize));

	Set("FractureEditor.Slice",                new IMAGE_BRUSH_SVG("FractureSliceCut", IconSize));
	Set("FractureEditor.Uniform",              new IMAGE_BRUSH_SVG("FractureUniformVoronoi", IconSize));
	Set("FractureEditor.Radial",               new IMAGE_BRUSH_SVG("FractureRadialVoronoi", IconSize));
	Set("FractureEditor.Clustered",            new IMAGE_BRUSH_SVG("FractureClusterVoronoi", IconSize));
	Set("FractureEditor.Planar",               new IMAGE_BRUSH_SVG("FracturePlanarCut", IconSize));
	Set("FractureEditor.Brick",                new IMAGE_BRUSH_SVG("FractureBrick", IconSize));
	Set("FractureEditor.Texture",              new IMAGE_BRUSH_SVG("FractureTexture", IconSize));
	Set("FractureEditor.Mesh",                 new IMAGE_BRUSH_SVG("FractureMesh", IconSize));

	// This is a bit of magic.  When you pass a command your Builder.AddToolBarButton, it will automatically try to find 
	// and Icon with the same name as the command and TCommand<> Context Name.  
	Set("FractureEditor.SelectAll",            new IMAGE_BRUSH_SVG("FractureSelectAll", IconSize));
	Set("FractureEditor.SelectNone",           new IMAGE_BRUSH_SVG("FractureDeselectAll", IconSize));
	Set("FractureEditor.SelectNeighbors",      new IMAGE_BRUSH_SVG("FractureSelectNeighbor", IconSize));
	Set("FractureEditor.SelectParent",		   new IMAGE_BRUSH_SVG("FractureSelectParent", IconSize));
	Set("FractureEditor.SelectChildren",       new IMAGE_BRUSH_SVG("FractureSelectChildren", IconSize));
	Set("FractureEditor.SelectSiblings",       new IMAGE_BRUSH_SVG("FractureSelectSiblings", IconSize));
	Set("FractureEditor.SelectAllInLevel",     new IMAGE_BRUSH_SVG("FractureSelectLevel", IconSize));
	Set("FractureEditor.SelectInvert",         new IMAGE_BRUSH_SVG("FractureSelectInvert", IconSize));
	Set("FractureEditor.SelectCustom",         new IMAGE_BRUSH_SVG("FractureSelectInteractive", IconSize));
	Set("FractureEditor.SelectLeaf",           new IMAGE_BRUSH_SVG("FractureSelectLeafNodes", IconSize));
	Set("FractureEditor.SelectCluster",        new IMAGE_BRUSH_SVG("FractureSelectClusters", IconSize));

	Set("FractureEditor.AutoCluster",          new IMAGE_BRUSH_SVG("FractureAutoCluster", IconSize));
	Set("FractureEditor.ClusterMagnet",        new IMAGE_BRUSH_SVG("FractureMagnet", IconSize));
	Set("FractureEditor.Cluster",              new IMAGE_BRUSH_SVG("FractureCluster", IconSize));
	Set("FractureEditor.Uncluster",            new IMAGE_BRUSH_SVG("FractureUncluster", IconSize));
	Set("FractureEditor.FlattenToLevel",       new IMAGE_BRUSH_SVG("FractureFlattenToLevel", IconSize));
	Set("FractureEditor.Flatten",              new IMAGE_BRUSH_SVG("FractureFlatten", IconSize));
	Set("FractureEditor.Merge",                new IMAGE_BRUSH_SVG("FractureMerge", IconSize));
	Set("FractureEditor.MoveUp",               new IMAGE_BRUSH_SVG("FractureMoveUpRow", IconSize));

	Set("FractureEditor.AddEmbeddedGeometry",  new IMAGE_BRUSH_SVG("FractureEmbed", IconSize));
	Set("FractureEditor.AutoEmbedGeometry",    new IMAGE_BRUSH_SVG("FractureAutoEmbed", IconSize));
	Set("FractureEditor.FlushEmbeddedGeometry",new IMAGE_BRUSH_SVG("FractureFlush", IconSize));

	Set("FractureEditor.AutoUV",               new IMAGE_BRUSH_SVG("FractureAutoUV", IconSize));
	Set("FractureEditor.Material",             new IMAGE_BRUSH("EditMats_40x", IconSize));

	Set("FractureEditor.DeleteBranch",         new IMAGE_BRUSH_SVG("FracturePrune", IconSize));
	Set("FractureEditor.Hide",				   new IMAGE_BRUSH_SVG("FractureHide", IconSize));
	Set("FractureEditor.Unhide",               new IMAGE_BRUSH_SVG("FractureUnhide", IconSize));
	Set("FractureEditor.MergeSelected",       new IMAGE_BRUSH_SVG("FractureGlue", IconSize));

	Set("FractureEditor.RecomputeNormals",     new IMAGE_BRUSH_SVG("FractureNormals", IconSize));
	Set("FractureEditor.Resample",             new IMAGE_BRUSH_SVG("FractureResample", IconSize));

	Set("FractureEditor.ToMesh",        	   new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/MakeStaticMesh", IconSize));
	Set("FractureEditor.Validate",        	   new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", IconSize));

	Set("FractureEditor.Convex",        	   new IMAGE_BRUSH_SVG("FractureConvex", IconSize));
	Set("FractureEditor.CustomVoronoi",        new IMAGE_BRUSH_SVG("FractureCustom", IconSize));
	Set("FractureEditor.FixTinyGeo",	       new IMAGE_BRUSH_SVG("FractureGeoMerge", IconSize));
	Set("FractureEditor.SetRemoveOnBreak",     new IMAGE_BRUSH_SVG("FractureSetRemoval", IconSize));
	Set("FractureEditor.Proximity",            new IMAGE_BRUSH_SVG("FractureProximity", IconSize));

	// View Settings
	Set("FractureEditor.Exploded",             new IMAGE_BRUSH_SVG("FractureMiniExploded", LabelIconSize));
	Set("FractureEditor.Levels",               new IMAGE_BRUSH_SVG("FractureMiniLevel", LabelIconSize));
	Set("FractureEditor.Visibility",           new IMAGE_BRUSH_SVG("FractureGeneralVisibility", IconSize));
	Set("FractureEditor.ToggleShowBoneColors", new IMAGE_BRUSH_SVG("FractureGeneralVisibility", IconSize));
	Set("FractureEditor.ViewUpOneLevel",       new IMAGE_BRUSH_SVG("FractureLevelViewUp", IconSize));
	Set("FractureEditor.ViewDownOneLevel",     new IMAGE_BRUSH_SVG("FractureLevelViewDown", IconSize));

	Set("FractureEditor.SetInitialDynamicState", new IMAGE_BRUSH_SVG("FractureState", IconSize));

	Set("FractureEditor.SpinBox", FSpinBoxStyle(FAppStyle::GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
		.SetTextPadding(FMargin(0))
		.SetBackgroundBrush(FSlateNoResource())
		.SetHoveredBackgroundBrush(FSlateNoResource())
		.SetInactiveFillBrush(FSlateNoResource())
		.SetActiveFillBrush(FSlateNoResource())
		.SetForegroundColor(FSlateColor::UseSubduedForeground())
		.SetArrowsImage(FSlateNoResource())
	);


	Set("FractureEditor.GenerateAsset", new IMAGE_BRUSH_SVG("FractureGenerateAsset", IconSize));
	Set("FractureEditor.ResetAsset", new IMAGE_BRUSH_SVG("FractureReset", FVector2D(20.0f, 20.0f)));

	Set("LevelEditor.FractureMode", new IMAGE_BRUSH_SVG("fracture", FVector2D(20.0f, 20.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FFractureEditorStyle::~FFractureEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FFractureEditorStyle& FFractureEditorStyle::Get()
{
	static FFractureEditorStyle Inst;
	return Inst;
}


