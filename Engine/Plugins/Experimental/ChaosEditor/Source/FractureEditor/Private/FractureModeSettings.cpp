// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureModeSettings.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionConvexPropertiesInterface.h"
#include "GeometryCollection/GeometryCollectionProximityPropertiesInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureModeSettings)

void UFractureModeSettings::ApplyDefaultConvexSettings(FGeometryCollection& GeometryCollection) const
{
	FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties Properties = GeometryCollection.GetConvexProperties();
	Properties.FractionRemove = ConvexFractionAllowRemove;
	Properties.SimplificationThreshold = ConvexSimplificationDistanceThreshold;
	Properties.CanExceedFraction = ConvexCanExceedFraction;
	Properties.RemoveOverlaps = ConvexRemoveOverlaps;
	Properties.OverlapRemovalShrinkPercent = ConvexOverlapRemovalShrinkPercent;
	GeometryCollection.SetConvexProperties(Properties);
}

void UFractureModeSettings::ApplyDefaultProximitySettings(FGeometryCollection& GeometryCollection) const
{
	FGeometryCollectionProximityPropertiesInterface::FProximityProperties Properties = GeometryCollection.GetProximityProperties();
	Properties.bUseAsConnectionGraph = bProximityUseAsConnectionGraph;
	Properties.DistanceThreshold = ProximityDistanceThreshold;
	Properties.Method = ProximityMethod;
	Properties.ContactMethod = ProximityContactMethod;
	Properties.ContactAreaMethod = ProximityConnectionContactAreaMethod;
	Properties.RequireContactAmount = ProximityContactThreshold;
	GeometryCollection.SetProximityProperties(Properties);
}

void UFractureModeSettings::ApplyDefaultSettings(FGeometryCollection& GeometryCollection) const
{
	ApplyDefaultProximitySettings(GeometryCollection);
	ApplyDefaultConvexSettings(GeometryCollection);
}
