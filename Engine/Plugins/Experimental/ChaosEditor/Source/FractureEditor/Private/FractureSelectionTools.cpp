// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureSelectionTools.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "EditorSupportDelegates.h"
#include "GeometryCollection/GeometryCollectionObject.h"

void FFractureSelectionTools::ToggleSelectedBones(UGeometryCollectionComponent* GeometryCollectionComponent, const TArray<int32>& BoneIndices, bool bClearCurrentSelection, bool bAdd, bool bSnapToLevel)
{
	if (GeometryCollectionComponent)
	{
		FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();

		if (const UGeometryCollection* MeshGeometryCollection = GeometryCollectionComponent->RestCollection)
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = MeshGeometryCollection->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				EditBoneColor.SetEnableBoneSelection(true);

				// if multiselect then append new BoneSelected to what is already selected, otherwise we just clear and replace the old selection with BoneSelected
				if (bClearCurrentSelection)
				{
					EditBoneColor.ResetBoneSelection();
				}
		
				EditBoneColor.ToggleSelectedBones(BoneIndices, bAdd, bSnapToLevel);
				EditBoneColor.SetHighlightedBones(EditBoneColor.GetSelectedBones(), true);
			}

			FEditorSupportDelegates::RedrawAllViewports.Broadcast();

		}
	}

}

void FFractureSelectionTools::ClearSelectedBones(UGeometryCollectionComponent* GeometryCollectionComponent)
{
	FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();
	EditBoneColor.ResetBoneSelection();
	EditBoneColor.ResetHighlightedBones();
}


void FFractureSelectionTools::SelectNeighbors(UGeometryCollectionComponent* GeometryCollectionComponent)
{
	if (GeometryCollectionComponent)
	{
		FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();
		EditBoneColor.SelectBones(GeometryCollection::ESelectionMode::Neighbors);
	}
}
