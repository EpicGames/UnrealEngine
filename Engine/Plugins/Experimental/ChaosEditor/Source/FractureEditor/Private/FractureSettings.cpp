// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureSettings)

#if WITH_EDITOR

UFractureSettings::UFractureSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, ExplodeAmount(0.0)
	, FractureLevel(-1)
	, bHideUnselected(false)
	, SelectionDisplayMode(EFractureSelectionDisplayMode::Highlight)
{}

#endif



