// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolAutoCluster.h"

#include "FractureEditorStyle.h"
#include "FractureEditorCommands.h"
#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Math/NumericLimits.h"
#include "FractureEngineClustering.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolAutoCluster)

#define LOCTEXT_NAMESPACE "FractureAutoCluster"


UFractureToolAutoCluster::UFractureToolAutoCluster(const FObjectInitializer& ObjInit)
  : Super(ObjInit) 
{
	AutoClusterSettings = NewObject<UFractureAutoClusterSettings>(GetTransientPackage(), UFractureAutoClusterSettings::StaticClass());
	AutoClusterSettings->OwnerTool = this;
}


// UFractureTool Interface
FText UFractureToolAutoCluster::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureAutoCluster", "FractureToolAutoCluster", "Auto")); 
}


FText UFractureToolAutoCluster::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureAutoCluster", "FractureToolAutoClusterToolTip", "Automatically group together pieces of a fractured mesh (based on your settings) and assign them within the Geometry Collection.")); 
}

FText UFractureToolAutoCluster::GetApplyText() const 
{ 
	return FText(NSLOCTEXT("FractureAutoCluster", "ExecuteAutoCluster", "Auto Cluster")); 
}

FSlateIcon UFractureToolAutoCluster::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.AutoCluster");
}


TArray<UObject*> UFractureToolAutoCluster::GetSettingsObjects() const 
{
	TArray<UObject*> AllSettings; 
	AllSettings.Add(AutoClusterSettings);
	return AllSettings;
}

void UFractureToolAutoCluster::DrawBox(FPrimitiveDrawInterface* PDI, FVector Center, float SideLength)
{
	float SideBy2 = SideLength/2.;

	FVector X(1., 0, 0);
	FVector Y(0, 1., 0);
	FVector Z(0, 0, 1.);

	TArray<FVector> Verts;
	for (int i=-1; i<=1; i+=2)
	{
		for (int j=-1; j<=1; j+=2)
		{
			for (int k=-1; k<=1; k+=2)
			{
				Verts.Emplace(Center + (X*i + Y*j + Z*k)*SideBy2);
			}
		}
	}

	Verts.Swap(0,1);
	Verts.Swap(4,5);

	for (int i=0; i<4; i+=1)
	{
		PDI->DrawLine(Verts[i], Verts[(i+1)%4], FLinearColor(255, 0, 0), SDPG_Foreground);
		PDI->DrawLine(Verts[4+i], Verts[4+((i+1)%4)], FLinearColor(255, 0, 0), SDPG_Foreground);
		PDI->DrawLine(Verts[i], Verts[i+4], FLinearColor(255, 0, 0), SDPG_Foreground);
	}
}

void UFractureToolAutoCluster::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (AutoClusterSettings->ClusterSizeMethod == EClusterSizeMethod::ByGrid)
	{
		EnumerateVisualizationMapping(GridPointsMappings, ShowGridPoints.Num(),
			[&](int32 Idx, FVector ExplodedVector)
			{
				FVector Pt = ShowGridPoints[Idx];
				PDI->DrawPoint(Pt, FLinearColor(1.f, 0.f, 1.f), 5, SDPG_Foreground);
			});
		if (AutoClusterSettings->bShowGridLines)
		{
			EnumerateVisualizationMapping(GridLinesMappings, ShowGridLines.Num(),
				[&](int32 Idx, FVector ExplodedVector)
				{
					TPair<FVector, FVector> Pts = ShowGridLines[Idx];
					PDI->DrawLine(Pts.Key, Pts.Value, FLinearColor(1.f, 1.f, 0.f), SDPG_Foreground);
				});
		}
	}

	if (AutoClusterSettings->ClusterSizeMethod == EClusterSizeMethod::BySize)
	{
		TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

		for (FFractureToolContext& FractureContext : FractureContexts)
		{
			DrawBox(PDI, FractureContext.GetTransform().GetTranslation(), AutoClusterSettings->SiteSize);
		}
	}
}

void UFractureToolAutoCluster::FractureContextChanged()
{
	ClearVisualizations();

	TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

	for (FFractureToolContext& Context : Contexts)
	{
		if (Context.GetGeometryCollection())
		{
			Context.ConvertSelectionToClusterNodes();

			if (const FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get())
			{
				int CollectionIdx = VisualizedCollections.Emplace(Context.GetGeometryCollectionComponent());
				FTransform ContextTransform = Context.GetTransform();

				for (const int32 ClusterIndex : Context.GetSelection())
				{
					GridPointsMappings.AddMapping(CollectionIdx, ClusterIndex, ShowGridPoints.Num());
					FBox Bounds;
					TArray<FVector> LocalGridPoints = FFractureEngineClustering::GenerateGridSites(
						*GeometryCollection, ClusterIndex,
						AutoClusterSettings->ClusterGridWidth,
						AutoClusterSettings->ClusterGridDepth,
						AutoClusterSettings->ClusterGridHeight,
						&Bounds);
					for (FVector LocalPt : LocalGridPoints)
					{
						ShowGridPoints.Add(ContextTransform.TransformPosition(LocalPt));
					}
					GridLinesMappings.AddMapping(CollectionIdx, ClusterIndex, ShowGridLines.Num());
					FIntVector3 Dims(
						AutoClusterSettings->ClusterGridWidth,
						AutoClusterSettings->ClusterGridDepth,
						AutoClusterSettings->ClusterGridHeight);
					FVector InvDims(1.0 / double(Dims.X), 1.0 / double(Dims.Y), 1.0 / double(Dims.Z));
					FVector Diag = Bounds.Max - Bounds.Min;
					for (int32 Dim = 0; Dim < 3; ++Dim)
					{
						FIntVector3 DMap(Dim, (Dim + 1) % 3, (Dim + 2) % 3);
						FVector Start, End;
						Start[DMap.Z] = Bounds.Min[DMap.Z];
						End[DMap.Z] = Bounds.Max[DMap.Z];
						for (int32 W = 0; W <= Dims[DMap.X]; ++W)
						{
							double AlongW = W * InvDims[DMap.X];
							Start[DMap.X] = Bounds.Min[DMap.X] + AlongW * Diag[DMap.X];
							End[DMap.X] = Start[DMap.X];
							for (int32 H = 0; H <= Dims[DMap.Y]; ++H)
							{
								double AlongH = H * InvDims[DMap.Y];
								Start[DMap.Y] = Bounds.Min[DMap.Y] + AlongH * Diag[DMap.Y];
								End[DMap.Y] = Start[DMap.Y];
								ShowGridLines.Emplace(ContextTransform.TransformPosition(Start), ContextTransform.TransformPosition(End));
							}
						}
					}
				}
			}
		}
	}
}

void UFractureToolAutoCluster::RegisterUICommand( FFractureEditorCommands* BindingContext )
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "AutoCluster", "Auto", "Automatically group pieces of a fractured mesh into a specified number of clusters.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->AutoCluster = UICommandInfo;
}

void UFractureToolAutoCluster::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();
		
		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				Context.ConvertSelectionToClusterNodes();
				if (FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get())
				{
					int32 StartTransformCount = GeometryCollection->Transform.Num();

					for (const int32 ClusterIndex : Context.GetSelection())
					{
						int32 KMeansIterations = AutoClusterSettings->ClusterSizeMethod == EClusterSizeMethod::ByGrid ? AutoClusterSettings->DriftIterations : 500;
						// note: bPreferConvexity is incompatible with grid clustering
						bool bPreferConvexity = AutoClusterSettings->bPreferConvexity && AutoClusterSettings->ClusterSizeMethod != EClusterSizeMethod::ByGrid;
						FFractureEngineClustering::AutoCluster(*GeometryCollection,
							ClusterIndex,
							(EFractureEngineClusterSizeMethod)AutoClusterSettings->ClusterSizeMethod,
							AutoClusterSettings->SiteCount,
							AutoClusterSettings->SiteCountFraction,
							AutoClusterSettings->SiteSize,
							AutoClusterSettings->bEnforceConnectivity,
							AutoClusterSettings->bAvoidIsolated,
							AutoClusterSettings->bEnforceSiteParameters,
							AutoClusterSettings->ClusterGridWidth,
							AutoClusterSettings->ClusterGridDepth,
							AutoClusterSettings->ClusterGridHeight,
							AutoClusterSettings->MinimumSize,
							KMeansIterations,
							bPreferConvexity,
							AutoClusterSettings->ConcavityTolerance
						);
					}

					Context.GenerateGuids(StartTransformCount);
				}

				Refresh(Context, Toolkit);
			}
		}
		SetOutlinerComponents(Contexts, Toolkit);
	}
}
	
#undef LOCTEXT_NAMESPACE


