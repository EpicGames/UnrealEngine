// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "Algo/Count.h"

#include "FractureToolAutoCluster.generated.h"

// Note: Only Voronoi-based auto-clustering is currently supported
UENUM()
enum class EFractureAutoClusterMode : uint8
{
	/** Overlapping bounding box*/
	BoundingBox UMETA(DisplayName = "Bounding Box"),

	/** GC connectivity */
	Proximity UMETA(DisplayName = "Proximity"),

	/** Distance */
	Distance UMETA(DisplayName = "Distance"),

	Voronoi UMETA(DisplayName = "Voronoi"),
};

UENUM()
enum class EClusterSizeMethod : uint8
{
	// Cluster by specifying an absolute number of clusters
	ByNumber,
	// Cluster by specifying a fraction of the number of input bones
	ByFractionOfInput,
	// Cluster by specifying the density of the input bones
	BySize,
	// Cluster by a grid layout
	ByGrid,
};


UCLASS(DisplayName = "Auto Cluster", Category = "FractureTools")
class UFractureAutoClusterSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureAutoClusterSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Simplified interface now only supports Voronoi clustering."))
	EFractureAutoClusterMode AutoClusterMode_DEPRECATED;

	/** How to choose the size of the clusters to create */
	UPROPERTY(EditAnywhere, Category = ClusterSize)
	EClusterSizeMethod ClusterSizeMethod = EClusterSizeMethod::ByNumber;

	/** Use a Voronoi diagram with this many Voronoi sites as a guide for deciding cluster boundaries */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DisplayName = "Cluster Sites", UIMin = "2", UIMax = "5000", ClampMin = "1", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByNumber", EditConditionHides))
	uint32 SiteCount=10;

	/** Choose the number of Voronoi sites used for clustering as a fraction of the number of child bones to process */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DisplayName = "Cluster Fraction", ClampMin = "0", ClampMax = ".5", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByFractionOfInput", EditConditionHides))
	float SiteCountFraction = .25;
	
	/** Choose the Edge-Size of the cube used to groups bones under a cluster (in cm). */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DisplayName = "Cluster Size", UIMin = ".01", UIMax = "100", ClampMin = ".0001", ClampMax = "10000", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::BySize", EditConditionHides))
	float SiteSize = 1;

	/** Choose the number of cluster sites to distribute along the X axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByGrid", EditConditionHides))
	int ClusterGridWidth = 2;

	/** Choose the number of cluster sites to distribute along the Y axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByGrid", EditConditionHides))
	int ClusterGridDepth = 2;
	
	/** Choose the number of cluster sites to distribute along the Z axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByGrid", EditConditionHides))
	int ClusterGridHeight = 2;

	/** If true, show the cluster grid boundary lines. */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByGrid", EditConditionHides))
	bool bShowGridLines = true;

	/** If a cluster has volume less than this value (in cm) cubed, then the auto-cluster process will attempt to merge it into a neighboring cluster. */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (ClampMin = "0"))
	float MinimumSize = 0;

	/** For a grid distribution, optionally iteratively recenter the grid points to the center of the cluster geometry (technically: applying K-Means iterations) to balance the shape and distribution of the clusters */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DisplayName = "Drift Iterations", ClampMin = "0", UIMax = "5", EditCondition = "ClusterSizeMethod == EClusterSizeMethod::ByGrid", EditConditionHides))
	int DriftIterations = 0;

	/** Whether to favor clusters that have a convex shape. (Note: Does not support ByGrid clustering.)  */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "ClusterSizeMethod != EClusterSizeMethod::ByGrid"))
	bool bPreferConvexity = false;

	/** If > 0, cube root of maximum concave volume to add per cluster (ignoring concavity of individual parts) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "bPreferConvexity && ClusterSizeMethod != EClusterSizeMethod::ByGrid"))
	double ConcavityTolerance = 0;

	/** If true, bones will only be added to the same cluster if they are physically connected (either directly, or via other bones in the same cluster) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (DisplayName = "Enforce Cluster Connectivity"))
	bool bEnforceConnectivity=true;

	/** If true, make sure the site parameters are matched as close as possible ( bEnforceConnectivity can make the number of site larger than the requested input may produce without it ) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "bEnforceConnectivity == true && ClusterSizeMethod != EClusterSizeMethod::ByGrid && !bPreferConvexity"))
	bool bEnforceSiteParameters = true;

	/** If true, prevent the creation of clusters with only a single child, skipping creation of a new cluster in such cases. */
	UPROPERTY(EditAnywhere, Category = AutoCluster)
	bool bAvoidIsolated = true;
};


UCLASS(DisplayName="AutoCluster", Category="FractureTools")
class UFractureToolAutoCluster: public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolAutoCluster(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override; 
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext );
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	void DrawBox(FPrimitiveDrawInterface* PDI, FVector Center, float SideLength);

	virtual void FractureContextChanged() override;

	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	UPROPERTY(EditAnywhere, Category = AutoCluster)
	TObjectPtr<UFractureAutoClusterSettings> AutoClusterSettings;

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		ShowGridPoints.Empty();
		GridPointsMappings.Empty();
		ShowGridLines.Empty();
		GridLinesMappings.Empty();
	}

private:

	TArray<FVector> ShowGridPoints;
	TArray<TPair<FVector, FVector>> ShowGridLines;
	FVisualizationMappings GridPointsMappings, GridLinesMappings;
};

