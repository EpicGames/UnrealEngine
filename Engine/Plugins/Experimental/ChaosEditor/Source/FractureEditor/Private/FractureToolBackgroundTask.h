// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/GeometryCollection.h"
#include "ModelingTaskTypes.h"
#include "ModelingOperators.h"
#include "BackgroundModelingComputeSource.h"
#include "Misc/ScopedSlowTask.h"

namespace UE
{
namespace Fracture
{

// Base class for background operators that update geometry collections (e.g. to fracture in a background thread)
template<typename ResultType>
class TGeometryCollectionOperator : public UE::Geometry::TGenericDataOperator<ResultType>
{
public:
	int ResultGeometryIndex = -1;
	TUniquePtr<FGeometryCollection> CollectionCopy;

	TGeometryCollectionOperator(const FGeometryCollection& SourceCollection) : UE::Geometry::TGenericDataOperator<ResultType>(false)
	{
		CollectionCopy = MakeUnique<FGeometryCollection>();
		CollectionCopy->CopyMatchingAttributesFrom(SourceCollection, nullptr);
	}

	virtual ~TGeometryCollectionOperator() = default;

	virtual int GetResultGeometryIndex()
	{
		return ResultGeometryIndex;
	}

	// Post-process the geometry collection on success
	virtual void OnSuccess(ResultType& Collection)
	{

	}

	// For operators where there is no geometry index to report, set a generic success value
	virtual void SetSuccessIndex()
	{
		ResultGeometryIndex = 0;
	}
};

// Most operators will also have FGeometryCollection as their result
using FGeometryCollectionOperator = TGeometryCollectionOperator<FGeometryCollection>;

// Fracture-specific operators also clear proximity on success
class FGeometryCollectionFractureOperator : public FGeometryCollectionOperator
{
public:
	FGeometryCollectionFractureOperator(const FGeometryCollection& SourceCollection) : FGeometryCollectionOperator(SourceCollection)
	{
	}

	virtual ~FGeometryCollectionFractureOperator() = default;

	virtual void OnSuccess(FGeometryCollection& Collection) override
	{
		// Invalidate proximity
		if (Collection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
		{
			Collection.RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
		}
	}
};

template<class OpType>
using TBackgroundOpExecuter = UE::Geometry::FAsyncTaskExecuterWithProgressCancel<UE::Geometry::TModelingOpTask<OpType>>;


// Start a background task
// Note: If automatically starting tasks in response to user actions, consider using a timeout to prevent launching too many tasks at once
// (TODO: consider encapsulating the timeout logic in a small struct here)
template<class OpType>
TUniquePtr<TBackgroundOpExecuter<OpType>> StartBackgroundTask(TUniquePtr<OpType>&& NewOp)
{
	TUniquePtr<TBackgroundOpExecuter<OpType>> BackgroundTask = MakeUnique<TBackgroundOpExecuter<OpType>>(MoveTemp(NewOp));
	BackgroundTask->StartBackgroundTask();
	return BackgroundTask;
}

// Tick a background task started by StartBackgroundTask (above)
// @return the result of SuccessCallback if the task has completed, false otherwise
template<class OpType>
bool TickBackgroundTask(TUniquePtr<TBackgroundOpExecuter<OpType>>& BackgroundTask, bool bCancel, TFunctionRef<bool(TUniquePtr<typename OpType::ResultTypeName>&&)> SuccessCallback)
{
	bool bSuccess = false;
	if (bCancel)
	{
		BackgroundTask.Release()->CancelAndDelete();
	}
	else if (BackgroundTask->IsDone())
	{
		bSuccess = !BackgroundTask->GetTask().IsAborted();
	}

	if (bSuccess)
	{
		check(BackgroundTask != nullptr && BackgroundTask->IsDone());
		TUniquePtr<OpType> Op = BackgroundTask->GetTask().ExtractOperator();
		if (!ensure(Op))
		{
			return false;
		}

		TUniquePtr<typename OpType::ResultTypeName> Result = Op->ExtractResult();
		if (!Result.IsValid())
		{
			return false;
		}
		bool bResult = SuccessCallback(MoveTemp(Result));
		BackgroundTask.Reset();
		return bResult;
	}

	return false;
}

// Cancel the background task (or ignore if the pointer is empty)
template<class OpType>
void CancelBackgroundTask(TUniquePtr<TBackgroundOpExecuter<OpType>>& BackgroundTask)
{
	if (BackgroundTask)
	{
		BackgroundTask.Release()->CancelAndDelete();
	}
}


// Run a blocking geometry collection op, but with a responsive cancel option
template<class GeometryCollectionOpType, typename ResultType>
int RunCancellableGeometryCollectionOpGeneric(ResultType& ToUpdate, TFunctionRef<void(ResultType&, ResultType&)> AssignResult, TUniquePtr<GeometryCollectionOpType>&& NewOp, FText DefaultMessage, float DialogDelay = .5)
{
	using FGeometryCollectionTask = UE::Geometry::TModelingOpTask<GeometryCollectionOpType>;
	using FExecuter = UE::Geometry::FAsyncTaskExecuterWithProgressCancel<FGeometryCollectionTask>;
	TUniquePtr<FExecuter> BackgroundTask = MakeUnique<FExecuter>(MoveTemp(NewOp));
	BackgroundTask->StartBackgroundTask();

	FScopedSlowTask SlowTask(1, DefaultMessage);
	SlowTask.MakeDialogDelayed(DialogDelay, true);

	bool bSuccess = false;
	while (true)
	{
		if (SlowTask.ShouldCancel())
		{
			// Release ownership to the TDeleterTask that is spawned by CancelAndDelete()
			BackgroundTask.Release()->CancelAndDelete();
			break;
		}
		if (BackgroundTask->IsDone())
		{
			bSuccess = !BackgroundTask->GetTask().IsAborted();
			break;
		}
		FPlatformProcess::Sleep(.2f); // SlowTask::ShouldCancel will throttle any updates faster than .2 seconds
		float ProgressFrac;
		FText ProgressMessage;
		bool bMadeProgress = BackgroundTask->PollProgress(ProgressFrac, ProgressMessage);
		if (bMadeProgress)
		{
			// SlowTask expects progress to be reported before it happens; we work around this by directly updating the progress amount
			SlowTask.CompletedWork = ProgressFrac;
			SlowTask.EnterProgressFrame(0, ProgressMessage);
		}
		else
		{
			SlowTask.TickProgress(); // Still tick the UI when we don't get new progress frames
		}
	}

	if (bSuccess)
	{
		check(BackgroundTask != nullptr && BackgroundTask->IsDone());
		TUniquePtr<GeometryCollectionOpType> Op = BackgroundTask->GetTask().ExtractOperator();

		TUniquePtr<ResultType> Result = Op->ExtractResult();
		if (!Result.IsValid())
		{
			return -1;
		}

		AssignResult(ToUpdate, *Result);

		Op->OnSuccess(ToUpdate);
		return Op->GetResultGeometryIndex();
	}

	return -1;
}

template<class GeometryCollectionOpType>
int RunCancellableGeometryCollectionOp(FGeometryCollection& ToUpdate, TUniquePtr<GeometryCollectionOpType>&& NewOp, FText DefaultMessage, float DialogDelay = .5)
{
	return RunCancellableGeometryCollectionOpGeneric<GeometryCollectionOpType, FGeometryCollection>(
		ToUpdate, [](FGeometryCollection& ToUpdateIn, FGeometryCollection& ResultIn)
		{
			ToUpdateIn.CopyMatchingAttributesFrom(ResultIn, nullptr);
		}, MoveTemp(NewOp), DefaultMessage, DialogDelay);
}

} // end namespace UE::Fracture
} // end namespace UE