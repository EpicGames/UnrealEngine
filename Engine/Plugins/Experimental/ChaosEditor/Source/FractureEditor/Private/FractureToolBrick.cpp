// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolBrick.h"

#include "FractureEditorModeToolkit.h"
#include "FractureEditorStyle.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "FractureEditorStyle.h"
#include "FractureEngineFracturing.h"
#include "GameFramework/Actor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "PlanarCut.h"
#include "FractureToolContext.h"
#include "FractureToolBackgroundTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolBrick)

using namespace UE::Fracture;

#define LOCTEXT_NAMESPACE "FractureBrick"


UFractureToolBrick::UFractureToolBrick(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	BrickSettings = NewObject<UFractureBrickSettings>(GetTransientPackage(), UFractureBrickSettings::StaticClass());
	BrickSettings->OwnerTool = this;
}

FText UFractureToolBrick::GetDisplayText() const
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolBrick", "Brick Fracture")); 
}

FText UFractureToolBrick::GetTooltipText() const 
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolBrickTooltip", "This type of fracture enables you to define a pattern to perform the fracture, along with the forward and up axis in which to fracture. You can also adjust the brick length, height, or depth to provide varying results.  Click the Fracture Button to commit the fracture to the geometry collection.")); 
}

FSlateIcon UFractureToolBrick::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Brick");
}

void UFractureToolBrick::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Brick", "Brick", "Fracture with a customizable brick pattern. Note: Currently only supports fracturing with at least some (non-zero) Grout.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Brick = UICommandInfo;
}

TArray<UObject*> UFractureToolBrick::GetSettingsObjects() const 
{ 
	TArray<UObject*> Settings;
	Settings.Add(BrickSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

namespace FractureToolBrickLocals
{
	// Calculate total number of bricks based on given dimensions and the extent of the object to be fractured.
	// If the input is not valid or the result is too large, this functions returns -1.
	// It is possible that we are dealing with incredibly large meshes and small brick dimensions. Doing the
	// calculations in double and checking for NaNs will catch cases where for integer we would have to jump through
	// multiple hoops to make sure we are not dealing with overflow. And since the limit for the number of bricks is
	// comparably low, we do not need to worry about loss in precision for very large integers.
	// For example, running this calculation with brick dimensions of 1 for the Sky Sphere will result in integer
	// overflow.
	static int64 CalculateNumBricks(const FVector& Dimensions, const FVector& Extents)
	{
		if (Dimensions.GetMin() <= 0 || Extents.GetMin() <= 0)
		{
			return -1;
		}

		const FVector NumBricksPerDim(ceil(Extents.X / Dimensions.X), ceil(Extents.Y / Dimensions.Y), ceil(Extents.Z / Dimensions.Z));
		if (NumBricksPerDim.ContainsNaN())
		{
			return -1;
		}
		
		const double NumBricks = NumBricksPerDim.X * NumBricksPerDim.Y * NumBricksPerDim.Z;
		if (FMath::IsNaN(NumBricks))
		{
			return -1;
		}

		return static_cast<int64>(NumBricks);
	}

	static FVector GetBrickDimensions(const UFractureBrickSettings* BrickSettings, const FVector& Extents)
	{
		// Limit for the total number of bricks.
		const int64 NumBricksLimit = 8192;

		FVector Dimensions(BrickSettings->BrickLength, BrickSettings->BrickDepth, BrickSettings->BrickHeight);
		
		// Early out if we have inputs we cannot deal with. If this call to CalculateNumBricks is fine then any other
		// call to it will be fine, too, and we do not need to check for invalid results again.
		int64 NumBricks = CalculateNumBricks(Dimensions, Extents);
		if (NumBricks < 0)
		{
			return FVector::ZeroVector;
		}

		if (NumBricks > NumBricksLimit)
		{
			const int64 InputNumBricks = NumBricks;

			// Determine dimensions safely within the brick limit by iteratively doubling the brick size.
			FVector SafeDimensions = Dimensions;
			int64 SafeNumBricks;
			do
			{
				SafeDimensions *= 2;
				SafeNumBricks = CalculateNumBricks(SafeDimensions, Extents);
			} while (SafeNumBricks > NumBricksLimit);

			// Maximize brick dimensions to fit within the brick limit via iterative interval halving.
			const int32 IterationsMax = 10;
			int32 Iterations = 0;
			do
			{
				const FVector MidDimensions = (Dimensions + SafeDimensions) / 2;
				const int64 MidNumBricks = CalculateNumBricks(MidDimensions, Extents);

				if (MidNumBricks > NumBricksLimit)
				{
					Dimensions = MidDimensions;
					NumBricks = MidNumBricks;
				}
				else
				{
					SafeDimensions = MidDimensions;
					SafeNumBricks = MidNumBricks;
				}
			} while (++Iterations < IterationsMax);

			Dimensions = SafeDimensions;
			NumBricks = SafeNumBricks;

			UE_LOGF(LogFractureTool, Warning, "Brick Fracture: Current brick dimensions of %f x %f x %f would result in %lld bricks. "
				"Reduced brick dimensions to %f x %f x %f resulting in %lld bricks to stay within maximum number of %lld bricks.",
				BrickSettings->BrickLength, BrickSettings->BrickDepth, BrickSettings->BrickHeight, InputNumBricks,
				Dimensions.X, Dimensions.Y, Dimensions.Z, NumBricks, NumBricksLimit);
		}

		return Dimensions;
	}
}

void UFractureToolBrick::GenerateBrickTransforms(const FBox& Bounds)
{
	FFractureEngineFracturing::GenerateBrickTransforms(Bounds,
		BrickTransforms,
		static_cast<EFractureBrickBondEnum>(BrickSettings->Bond),
		BrickSettings->BrickLength, BrickSettings->BrickHeight, BrickSettings->BrickDepth,
		Edges);
}

void UFractureToolBrick::UpdateBrickTransforms()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		if (!FractureContext.GetBounds().IsValid) // skip contexts w/ invalid bounds
		{
			continue;
		}
		int32 CollectionIdx = VisualizedCollections.Emplace(FractureContext.GetGeometryCollectionComponent());
		int32 BoneIdx = FractureContext.GetSelection().Num() == 1 ? FractureContext.GetSelection()[0] : INDEX_NONE;
		BricksMappings.AddMapping(CollectionIdx, BoneIdx, BrickTransforms.Num());
		EdgesMappings.AddMapping(CollectionIdx, BoneIdx, Edges.Num());

		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

		const FBox Bounds = FractureContext.GetWorldBounds();
		GenerateBrickTransforms(Bounds);
	}
}

void UFractureToolBrick::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	UpdateBrickTransforms();
}

void UFractureToolBrick::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	UpdateBrickTransforms();
}

void UFractureToolBrick::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (CutterSettings->bDrawSites)
	{
		EnumerateVisualizationMapping(BricksMappings, BrickTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			const FTransform& Transform = BrickTransforms[Idx];
			PDI->DrawPoint(Transform.GetLocation() + ExplodedVector, FLinearColor::Green, 4.f, SDPG_Foreground);
		});
	}

	if (CutterSettings->bDrawDiagram)
	{
		PDI->AddReserveLines(SDPG_Foreground, Edges.Num(), false, false);
		EnumerateVisualizationMapping(EdgesMappings, Edges.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			PDI->DrawLine(Edges[Idx].Get<0>() + ExplodedVector, Edges[Idx].Get<1>() + ExplodedVector, FLinearColor(255, 0, 0), SDPG_Foreground);
		});
	}

}

void UFractureToolBrick::AddBoxEdges(const FVector& Min, const FVector& Max)
{
	Edges.Emplace(MakeTuple(Min, FVector(Min.X, Max.Y, Min.Z)));
	Edges.Emplace(MakeTuple(Min, FVector(Min.X, Min.Y, Max.Z)));
	Edges.Emplace(MakeTuple(FVector(Min.X, Max.Y, Max.Z), FVector(Min.X, Max.Y, Min.Z)));
	Edges.Emplace(MakeTuple(FVector(Min.X, Max.Y, Max.Z), FVector(Min.X, Min.Y, Max.Z)));

	Edges.Emplace(MakeTuple(FVector(Max.X, Min.Y, Min.Z), FVector(Max.X, Max.Y, Min.Z)));
	Edges.Emplace(MakeTuple(FVector(Max.X, Min.Y, Min.Z), FVector(Max.X, Min.Y, Max.Z)));
	Edges.Emplace(MakeTuple(Max, FVector(Max.X, Max.Y, Min.Z)));
	Edges.Emplace(MakeTuple(Max, FVector(Max.X, Min.Y, Max.Z)));

	Edges.Emplace(MakeTuple(Min, FVector(Max.X, Min.Y, Min.Z)));
	Edges.Emplace(MakeTuple(FVector(Min.X, Min.Y, Max.Z), FVector(Max.X, Min.Y, Max.Z)));
	Edges.Emplace(MakeTuple(FVector(Min.X, Max.Y, Min.Z), FVector(Max.X, Max.Y, Min.Z)));
	Edges.Emplace(MakeTuple(FVector(Min.X, Max.Y, Max.Z), Max));
}

class FCellsFractureOp : public FGeometryCollectionFractureOperator
{
public:
	FCellsFractureOp(const FGeometryCollection& SourceCollection) : FGeometryCollectionFractureOperator(SourceCollection)
	{}

	virtual ~FCellsFractureOp() = default;

	TArray<int> Selection;
	FPlanarCells Cells;
	FVector CellsOrigin;
	float PointSpacing;
	float Grout = 0;
	int Seed;
	FTransform Transform;
	bool bSplitIslands = true;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		ResultGeometryIndex = CutMultipleWithPlanarCells(Cells, *CollectionCopy, Selection, Grout, PointSpacing, Seed, Transform, true, true, Progress, CellsOrigin, FIslandSplitSettings(bSplitIslands, CloseVertexDistance, VertexToSurfaceBridgeDistance));

		SetResult(MoveTemp(CollectionCopy));
	}
};

int32 UFractureToolBrick::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
 		BrickTransforms.Empty();

		const FBox Bounds = FractureContext.GetWorldBounds();
		GenerateBrickTransforms(Bounds);

		// Get the same brick dimensions that were used in GenerateBrickTransform.
		// If we cannot deal with the input data then the brick dimensions will be zero, but we do not need to
		// explicitly handle that since it will only affect some local variables. The BrickTransforms will be empty
		// and there are no further side effects.
		const FVector BrickDimensions = FractureToolBrickLocals::GetBrickDimensions(BrickSettings, Bounds.Max - Bounds.Min);
		const FVector BrickHalfDimensions(BrickDimensions * 0.5);

		const FQuat HeaderRotation(FVector::UpVector, 1.5708);

		TArray<FBox> BricksToCut;

		// space the bricks by the grout setting, constrained to not erase the bricks
		const float MinDim = FMath::Min3(BrickHalfDimensions.X, BrickHalfDimensions.Y, BrickHalfDimensions.Z);
		const float HalfGrout = FMath::Clamp(0.5f * CutterSettings->Grout, 0, MinDim * 0.98f);
		const FVector HalfBrick(BrickHalfDimensions - HalfGrout);
		const FBox BrickBox(-HalfBrick, HalfBrick);

		FTransform ContextTransform = FractureContext.GetTransform();
		FVector Origin = ContextTransform.GetTranslation();

		for (const FTransform& Trans : BrickTransforms)
		{
			FTransform ToApply = Trans * FTransform(-Origin);
			BricksToCut.Add(BrickBox.TransformBy(ToApply));
		}

		TUniquePtr<FCellsFractureOp> BrickOp = MakeUnique<FCellsFractureOp>(*(FractureContext.GetGeometryCollection()));
		BrickOp->Selection = FractureContext.GetSelection();
		BrickOp->Grout = 0; // CutterSettings->Grout; // Note: Grout is currently baked directly into the brick cells above
		BrickOp->PointSpacing = CollisionSettings->GetPointSpacing();
		BrickOp->bSplitIslands = CutterSettings->bSplitIslands;
		BrickOp->CloseVertexDistance = CutterSettings->CloseVertexDistance;
		BrickOp->VertexToSurfaceBridgeDistance = CutterSettings->VertexToSurfaceBridgeDistance;
		const bool bBricksAreTouching = CutterSettings->Grout <= UE_KINDA_SMALL_NUMBER;
		BrickOp->Cells = FPlanarCells(BricksToCut, bBricksAreTouching);
		if (CutterSettings->Amplitude > 0.0f)
		{
			FNoiseSettings Settings;
			CutterSettings->TransferNoiseSettings(Settings);
			BrickOp->Cells.InternalSurfaceMaterials.NoiseSettings = Settings;
		}
		BrickOp->Seed = FractureContext.GetSeed();
		BrickOp->Transform = FractureContext.GetTransform();
		BrickOp->CellsOrigin = Origin;

		int Result = RunCancellableGeometryCollectionOp<FCellsFractureOp>(*(FractureContext.GetGeometryCollection()),
			MoveTemp(BrickOp), LOCTEXT("ComputingBrickFractureMessage", "Computing Brick Fracture"));
		return Result;
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE

