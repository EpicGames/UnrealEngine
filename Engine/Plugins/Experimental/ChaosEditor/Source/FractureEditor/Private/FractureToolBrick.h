// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "FractureToolCutter.h"

#include "FractureToolBrick.generated.h"

class FFractureToolContext;

// TODO: this enum was unused in the brick generation code; consider whether we want to keep + use it or not
///** Brick Projection Directions*/
//UENUM()
//enum class EFractureBrickProjection: uint8
//{
//	X UMETA(DisplayName = "X"),
//
//	Y UMETA(DisplayName = "Y"),
//
//	Z UMETA(DisplayName = "Z")
//};

/** Brick Stacking Pattern */
UENUM()
enum class EFractureBrickBond : uint8
{
	Stretcher UMETA(DisplayName = "Stretcher"),

	Stack UMETA(DisplayName = "Stack"),

	English  UMETA(DisplayName = "English"),

	Header UMETA(DisplayName = "Header"),

	Flemish UMETA(DisplayName = "Flemish")
};


UCLASS(config = EditorPerProjectUserSettings)
class UFractureBrickSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureBrickSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, Bond(EFractureBrickBond::Stretcher)
		, BrickLength(194.f)
		, BrickHeight(57.f)
		, BrickDepth(92.f)
	{}

	/** The brick bond pattern defines how the bricks are arranged */
	UPROPERTY(EditAnywhere, Category = Brick)
	EFractureBrickBond Bond;

	// Note: These Forward and Up parameters were unused in the actual brick generation; TODO: consider if we want to actually keep + use these parameters or not
	///** Forward Direction to project brick pattern. */
	//UPROPERTY(EditAnywhere, Category = Brick)
	//EFractureBrickProjection Forward = EFractureBrickProjection::X;
	///** Up Direction for vertical brick slices. */
	//UPROPERTY(EditAnywhere, Category = Brick)
	//EFractureBrickProjection Up = EFractureBrickProjection::Z;

	/** Brick length (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickLength;

	/** Brick height (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickHeight;

	/** Brick depth (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickDepth;
};


UCLASS(DisplayName="Brick", Category="FractureTools")
class UFractureToolBrick : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolBrick(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext ) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;

	void GenerateBrickTransforms(const FBox& Bounds);

	virtual void ConfigureCutterSettings() override
	{
		CutterSettings->bGroupFractureToggleEnabled = true;
		CutterSettings->bNoisePreviewToggleEnabled = false; // TODO: support noise preview for bricks
	}

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		BrickTransforms.Empty();
		Edges.Empty();
		Boxes.Empty();
		BricksMappings.Empty();
		EdgesMappings.Empty();
	}

private:
	UPROPERTY()
	TObjectPtr<UFractureBrickSettings> BrickSettings;

	void AddBoxEdges(const FVector& Min, const FVector& Max);

	void UpdateBrickTransforms();

	TArray<FTransform> BrickTransforms;
	TArray<TTuple<FVector, FVector>> Edges;
	TArray<TTuple<FVector, FVector>> Boxes;
	FVisualizationMappings BricksMappings, EdgesMappings;
};