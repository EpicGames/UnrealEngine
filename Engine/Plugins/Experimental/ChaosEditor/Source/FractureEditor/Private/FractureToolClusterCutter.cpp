// Copyright Epic Games, Inc. All Rights Reserved.


#include "FractureToolClusterCutter.h"

#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolClusterCutter)

#define LOCTEXT_NAMESPACE "FractureClustered"


UFractureToolClusterCutter::UFractureToolClusterCutter(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	ClusterSettings = NewObject<UFractureClusterCutterSettings>(GetTransientPackage(), UFractureClusterCutterSettings::StaticClass());
	ClusterSettings->OwnerTool = this;
}

FText UFractureToolClusterCutter::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolCluster", "Cluster Voronoi Fracture"));
}

FText UFractureToolClusterCutter::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolClusterTooltip", "Cluster Voronoi Fracture creates additional points around a base Voronoi pattern, creating more variation.  Click the Fracture Button to commit the fracture to the geometry collection."));
}

FSlateIcon UFractureToolClusterCutter::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Clustered");
}

void UFractureToolClusterCutter::RegisterUICommand( FFractureEditorCommands* BindingContext )
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Clustered", "Cluster", "Fracture using a Voronoi diagram with a 'clustered' pattern, creating varied fracture density across the shape.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Clustered = UICommandInfo;
}

TArray<UObject*> UFractureToolClusterCutter::GetSettingsObjects() const
{
	TArray<UObject*> ReturnSettings;
	ReturnSettings.Add(ClusterSettings);
	ReturnSettings.Add(CutterSettings);
	ReturnSettings.Add(CollisionSettings);
	return ReturnSettings;
}

void UFractureToolClusterCutter::GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites)
{
 	FRandomStream RandStream(Context.GetSeed());
	const int32 SiteCount = RandStream.RandRange(ClusterSettings->NumberClustersMin, ClusterSettings->NumberClustersMax);

	FBox Bounds = Context.GetWorldBounds();
	const FVector Extent(Bounds.Max - Bounds.Min);

	TArray<FVector> CenterSites;

	Sites.Reserve(Sites.Num() + SiteCount);
	for (int32 ii = 0; ii < SiteCount; ++ii)
	{
		CenterSites.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
	}

	for (int32 kk = 0, nk = CenterSites.Num(); kk < nk; ++kk)
	{
		const int32 SubSiteCount = RandStream.RandRange(ClusterSettings->SitesPerClusterMin, ClusterSettings->SitesPerClusterMax);

		for (int32 ii = 0; ii < SubSiteCount; ++ii)
		{
			FVector V(RandStream.VRand());
			V.Normalize();
			V *= ClusterSettings->ClusterRadiusOffset + (RandStream.FRandRange(ClusterSettings->ClusterRadiusFractionMin, ClusterSettings->ClusterRadiusFractionMax) * Bounds.GetExtent().GetAbsMax());
			V += CenterSites[kk];
			Sites.Emplace(V);
		}
	}
}

#undef LOCTEXT_NAMESPACE
