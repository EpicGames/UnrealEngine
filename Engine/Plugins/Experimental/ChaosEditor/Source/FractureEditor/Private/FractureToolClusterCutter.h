// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "FractureToolCutter.h"

#include "FractureToolClusterCutter.generated.h"

class FFractureToolContext;

UCLASS(config = EditorPerProjectUserSettings)
class UFractureClusterCutterSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureClusterCutterSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, NumberClustersMin(8)
		, NumberClustersMax(8)
		, SitesPerClusterMin(2)
		, SitesPerClusterMax(30)
		, ClusterRadiusFractionMin(0.1f)
		, ClusterRadiusFractionMax(0.2f)
		, ClusterRadiusOffset(0.0f)
	{}

	/** Minimum number of clusters of Voronoi sites to create. The amount of clusters created will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Min Num Clusters", UIMin = "1", UIMax = "200", ClampMin = "1"))
	int32 NumberClustersMin;

	/** Maximum number of clusters of Voronoi sites to create. The amount of clusters created will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Max Num Clusters", UIMin = "1", UIMax = "200", ClampMin = "1"))
	int32 NumberClustersMax;

	/** Minimum number of Voronoi sites per cluster. The amount of sites in each cluster will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Min Sites Per Cluster", UIMin = "0", UIMax = "200", ClampMin = "0"))
	int32 SitesPerClusterMin;

	/** Maximum number of Voronoi sites per cluster. The amount of sites in each cluster will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Max Sites Per Cluster", UIMin = "0", UIMax = "200", ClampMin = "0"))
	int32 SitesPerClusterMax;

	/**
	 * Minimum cluster radius (as fraction of the overall Voronoi diagram size). Cluster Radius Offset will be added to this
	 * Each Voronoi site will be placed at least this far (plus the Cluster Radius Offset) from its cluster center
	 */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Min Dist from Center (as Frac of Bounds)", UIMin = "0.0", UIMax = "1.0"))
	float ClusterRadiusFractionMin;

	/**
	 * Maximum cluster radius (as fraction of the overall Voronoi diagram size). Cluster Radius Offset will be added to this
	 * Each Voronoi site will be placed at most this far (plus the Cluster Radius Offset) from its cluster center
	 */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi, meta = (DisplayName = "Max Dist from Center (as Frac of Bounds)", UIMin = "0.0", UIMax = "1.0"))
	float ClusterRadiusFractionMax;

	/** Cluster radius offset (in cm). This offset will be added to the 'Min/Max Dist from Center' distance */
	UPROPERTY(EditAnywhere, Category = ClusterVoronoi)
	float ClusterRadiusOffset;
};


UCLASS(DisplayName="Cluster Voronoi", Category="FractureTools")
class UFractureToolClusterCutter : public UFractureToolVoronoiCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolClusterCutter(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext ) override;

	
	UPROPERTY(EditAnywhere, Category = Cluster)
	TObjectPtr<UFractureClusterCutterSettings> ClusterSettings;

protected:
	void GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites) override;

};