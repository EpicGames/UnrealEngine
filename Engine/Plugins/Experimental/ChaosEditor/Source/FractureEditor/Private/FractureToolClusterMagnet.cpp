// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolClusterMagnet.h"

#include "FractureTool.h"
#include "FractureEditorStyle.h"
#include "FractureEditorCommands.h"
#include "FractureToolContext.h"

#include "Chaos/TriangleMesh.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/Particles.h"
#include "Chaos/Vector.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include "FractureEngineClustering.h"

#include "GeometryCollection/GeometryCollectionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolClusterMagnet)


#define LOCTEXT_NAMESPACE "FractureClusterMagnet"



UFractureToolClusterMagnet::UFractureToolClusterMagnet(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	ClusterMagnetSettings = NewObject<UFractureClusterMagnetSettings>(GetTransientPackage(), UFractureClusterMagnetSettings::StaticClass());
	ClusterMagnetSettings->OwnerTool = this;
}


FText UFractureToolClusterMagnet::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolClusterMagnet", "Magnet"));
}


FText UFractureToolClusterMagnet::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolClusterMagnetToolTip", "Builds clusters by grouping the selected bones with their adjacent, neighboring bones. Can iteratively expand to a larger set of neighbors-of-neighbors."));
}

FText UFractureToolClusterMagnet::GetApplyText() const
{
	return FText(NSLOCTEXT("Fracture", "ExecuteClusterMagnet", "Cluster Magnet"));
}

FSlateIcon UFractureToolClusterMagnet::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.ClusterMagnet");
}


TArray<UObject*> UFractureToolClusterMagnet::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ClusterMagnetSettings);
	return Settings;
}

void UFractureToolClusterMagnet::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "ClusterMagnet", "Magnet", "Builds clusters by grouping the selected bones with their adjacent, neighboring bones. Can iteratively expand to a larger set of neighbors-of-neighbors.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->ClusterMagnet = UICommandInfo;
}


void UFractureToolClusterMagnet::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				TArray<int32> Selection = Context.GetSelection();
				int32 StartTransformCount = Context.GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup);
				FFractureEngineClustering::ClusterMagnet(
					*Context.GetGeometryCollection(),
					Selection,
					ClusterMagnetSettings->Iterations);

				Context.GenerateGuids(StartTransformCount);

				Refresh(Context, Toolkit);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}


#undef LOCTEXT_NAMESPACE
