// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolClusterMagnet.generated.h"

namespace Chaos
{
	template<class T, int d>
	class TParticles;
	using FParticles = TParticles<FReal, 3>;
}



UCLASS(DisplayName = "Cluster Magnet", Category = "FractureTools")
class UFractureClusterMagnetSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

		UFractureClusterMagnetSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, Iterations(1)
	{}

	/** How widely to search for neighboring bones to add to the new clusters: When Iterations is 1, only direct neighbors of the selected bones can be added to the new clusters; increasing Iterations allows additional layers of neighbors-of-neighbors to be added */
	UPROPERTY(EditAnywhere, Category = ClusterMagnet, meta = (DisplayName = "Iterations", UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500"))
	uint32 Iterations;

};


UCLASS(DisplayName = "ClusterMagnet", Category = "FractureTools")
class UFractureToolClusterMagnet : public UFractureModalTool
{
public:
	GENERATED_BODY()

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;
		
	UFractureToolClusterMagnet(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext);

	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	UPROPERTY(EditAnywhere, Category = AutoCluster)
	TObjectPtr<UFractureClusterMagnetSettings> ClusterMagnetSettings;

};
