// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolClustering.h"

#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "FractureEngineClustering.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolClustering)

#define LOCTEXT_NAMESPACE "FractureToolClusteringOps"

FText UFractureToolFlattenAll::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolFlattenAll", "Flatten"));
}

FText UFractureToolFlattenAll::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolFlattenAllTooltip", "Flattens all bones to level 1"));
}

FSlateIcon UFractureToolFlattenAll::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Flatten");
}

void UFractureToolFlattenAll::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Flatten", "Flatten", "Flattens all bones to level 1.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Flatten = UICommandInfo;
}

void UFractureToolFlattenAll::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("FlattenAll", "Flatten All"));

		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				if (!Context.GetGeometryCollection()->HasAttribute("Level", FGeometryCollection::TransformGroup))
				{
					FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(Edit.GetRestCollection()->GetGeometryCollection().Get(), -1);
				}
				const TManagedArray<int32>& Levels = Context.GetGeometryCollection()->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

				Context.Sanitize();
				Context.FilterSelectionBySimulationType(FGeometryCollection::FST_Clustered);

				int32 MaxClusterLevel = -1;
				for (int32 ClusterIndex : Context.GetSelection())
				{
					TArray<int32> LeafBones;
					MaxClusterLevel = FMath::Max(MaxClusterLevel, Levels[ClusterIndex]);
					FGeometryCollectionClusteringUtility::GetLeafBones(Context.GetGeometryCollection().Get(), ClusterIndex, true, LeafBones);
					FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(Context.GetGeometryCollection().Get(), ClusterIndex, LeafBones);
				}

				// if not viewing all levels, switch the view level to show make sure we can see the cluster levels
				if (Toolkit->GetLevelViewValue() != -1 && MaxClusterLevel != -1)
				{
					Toolkit->OnSetLevelViewValue(MaxClusterLevel + 1);
				}

				// Cleanup: Remove any clusters remaining in the flattened branch.
				FGeometryCollectionClusteringUtility::RemoveDanglingClusters(Context.GetGeometryCollection().Get());

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolCluster::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolCluster", "Cluster"));
}

FText UFractureToolCluster::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolClusterTooltip", "Clusters selected bones under a new parent."));
}

FSlateIcon UFractureToolCluster::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Cluster");
}

void UFractureToolCluster::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Cluster", "Cluster", "Clusters selected bones under a new parent.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Cluster = UICommandInfo;
}

void UFractureToolCluster::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("Cluster", "Cluster"));
		
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		int32 CurrentLevelView = Toolkit->GetLevelViewValue();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				if (FFractureEngineClustering::ClusterSelected(*Context.GetGeometryCollection(), Context.GetSelection()))
				{
					Refresh(Context, Toolkit);
				}
			}
		}

		if (CurrentLevelView != Toolkit->GetLevelViewValue())
		{
			Toolkit->OnSetLevelViewValue(CurrentLevelView);
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}	
}


FText UFractureToolUncluster::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolUncluster", "Uncluster"));
}

FText UFractureToolUncluster::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolUnclusterTooltip", "Remove parent cluster and move bones up a level."));
}

FSlateIcon UFractureToolUncluster::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Uncluster");
}

void UFractureToolUncluster::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Uncluster", "Unclstr", "Remove parent cluster and move bones up a level.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Uncluster = UICommandInfo;
}

void UFractureToolUncluster::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("Uncluster", "Uncluster"));
		
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				const TManagedArray<TSet<int32>>& Children = Context.GetGeometryCollection()->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
				if (!Context.GetGeometryCollection()->HasAttribute("Level", FGeometryCollection::TransformGroup))
				{
					FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(Edit.GetRestCollection()->GetGeometryCollection().Get(), -1);
				}
				const TManagedArray<int32>& Levels = Context.GetGeometryCollection()->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

				Context.ConvertSelectionToClusterNodes();
				Context.RemoveRootNodes();

				FGeometryCollectionClusteringUtility::CollapseHierarchyOneLevel(Context.GetGeometryCollection().Get(), Context.GetSelection());

				FGeometryCollectionClusteringUtility::RemoveDanglingClusters(Context.GetGeometryCollection().Get());
				Refresh(Context, Toolkit, true);
			}
		}
		
		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolMoveUp::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolMoveUp", "Level Up"));
}

FText UFractureToolMoveUp::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolMoveUpTooltip", "Move bones up a level."));
}

FSlateIcon UFractureToolMoveUp::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.MoveUp");
}

void UFractureToolMoveUp::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "MoveUp", "Level Up", "Move bones up a level.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->MoveUp = UICommandInfo;
}

void UFractureToolMoveUp::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("MovelUp", "Level Up"));
		
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context: Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				Context.ConvertEmbeddedSelectionToParents(); // it's not valid to 'move up' embedded geo
				Context.RemoveChildrenOfSelectedNodesFromSelection(); // if a node's children are also selected, just move up the parent node

				FGeometryCollectionClusteringUtility::MoveUpOneHierarchyLevel(Context.GetGeometryCollection().Get(), Context.GetSelection());
				FGeometryCollectionClusteringUtility::RemoveDanglingClusters(Context.GetGeometryCollection().Get());
				Refresh(Context, Toolkit, true);
			}
		}
		
		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolClusterMerge::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolClusterMerge", "Cluster Merge"));
}

FText UFractureToolClusterMerge::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolClusteringOps", "FractureToolClusterMergeTooltip", "Merge selected nodes into the same cluster."));
}

FSlateIcon UFractureToolClusterMerge::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Merge");
}

void UFractureToolClusterMerge::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Merge", "Merge", "Merge selected nodes into the same cluster.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->ClusterMerge = UICommandInfo;
}

void UFractureToolClusterMerge::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ClusterMerge", "Cluster Merge"));
		
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);

				if (FFractureEngineClustering::MergeSelectedClusters(*Context.GetGeometryCollection(), Context.GetSelection()))
				{
					Refresh(Context, Toolkit);
				}
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}

#undef LOCTEXT_NAMESPACE

