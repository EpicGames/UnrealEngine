// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolClustering.generated.h"


UCLASS(DisplayName = "Flatten All", Category = "FractureTools")
class UFractureToolFlattenAll : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolFlattenAll(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Cluster", Category = "FractureTools")
class UFractureToolCluster : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolCluster(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Uncluster", Category = "FractureTools")
class UFractureToolUncluster : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolUncluster(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "MoveUp", Category = "FractureTools")
class UFractureToolMoveUp : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolMoveUp(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "ClusterMerge", Category = "FractureTools")
class UFractureToolClusterMerge : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolClusterMerge(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};
