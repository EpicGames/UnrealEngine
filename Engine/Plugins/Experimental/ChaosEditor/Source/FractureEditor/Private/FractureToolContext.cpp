// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionUtility.h"

#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

#include "Algo/RemoveIf.h"


FFractureToolContext::FFractureToolContext(UGeometryCollectionComponent* InGeometryCollectionComponent)
{
	GeometryCollectionComponent = InGeometryCollectionComponent;
	FGeometryCollectionEdit RestCollection = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None);
	FracturedGeometryCollection = RestCollection.GetRestCollection();

	Transform = GeometryCollectionComponent->GetOwner()->GetActorTransform();

	if (UGeometryCollection* GeometryCollectionObject = RestCollection.GetRestCollection())
	{
		GeometryCollection = GeometryCollectionObject->GetGeometryCollection();
		SelectedBones = GeometryCollectionComponent->GetSelectedBones();
	}
}


void FFractureToolContext::Sanitize(bool bFavorParents)
{
	// Ensure that selected indices are valid
	if (GeometryCollection)
	{
		int NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
		SelectedBones.RemoveAll([this, NumTransforms](int32 Index) {
			return Index == INDEX_NONE || !ensure(Index < NumTransforms);
			});

		// Ensure that children of a selected node are not also selected.
		if (bFavorParents)
		{
			SelectedBones.RemoveAll([this](int32 Index) {
				return !IsValidBone(Index) || HasSelectedAncestor(Index);
				});
		}

		SelectedBones.Sort();
	}
}

void FFractureToolContext::RemoveRootNodes()
{
	// Ensure that selected indices are valid
	int NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
	SelectedBones.RemoveAll([this, NumTransforms](int32 Index) {
		return Index == INDEX_NONE || !ensure(Index < NumTransforms);
		});

	SelectedBones.RemoveAll([this](int32 Index) {
		return FGeometryCollectionClusteringUtility::IsARootBone(this->GeometryCollection.Get(), Index);
		});
}

void FFractureToolContext::GenerateGuids(int32 StartIdx)
{
	::GeometryCollection::GenerateTemporaryGuids(this->GeometryCollection.Get(), StartIdx, true);
}

TMap<int32, TArray<int32>> FFractureToolContext::GetClusteredSelections()
{
	TMap<int32, TArray<int32>> SiblingGroups;

	// Bin the selection indices by parent index
	const TManagedArray<int32>& Parents = GeometryCollection->GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
	for (int32 Index : SelectedBones)
	{
		TArray<int32>& SiblingIndices = SiblingGroups.FindOrAdd(Parents[Index]);
		SiblingIndices.Add(Index);
	}

	return SiblingGroups;
}

void FFractureToolContext::ConvertSelectionToLeafNodes()
{
	Sanitize();

	TArray<int32> LeafSelection;
	for (int32 Index : SelectedBones)
	{
		ConvertSelectionToLeafNodes(Index, LeafSelection);
	}

	SelectedBones = LeafSelection;
}

void FFractureToolContext::ConvertSelectionToLeafNodes(int32 Index, TArray<int32>& LeafSelection)
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
	if (Children[Index].Num() == 0)
	{
		LeafSelection.Add(Index);
	}
	else 
	{
		for (int32 Child : Children[Index])
		{
			ConvertSelectionToLeafNodes(Child, LeafSelection);
		}
	}	
}

void FFractureToolContext::RandomReduceSelection(float ProbToKeep)
{
	FRandomStream RandStream(GetSeed());
	for (int32 i = 0; i < SelectedBones.Num(); i++)
	{
		if (RandStream.GetFraction() >= ProbToKeep) // range does not include 1, so if ProbToKeep is 1 this will never remove
		{
			SelectedBones.RemoveAtSwap(i);
			i--;
		}
	}
}

void FFractureToolContext::RemoveChildrenOfSelectedNodesFromSelection()
{
	Sanitize();
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TSet<int32> SelectionSet(SelectedBones);
	SelectedBones.SetNum(Algo::RemoveIf(SelectedBones, [&](const int32& Bone)
	{
		int32 Parent = Bone;
		while ((Parent = Parents[Parent]) != INDEX_NONE)
		{
			if (SelectionSet.Contains(Parent))
			{
				return true;
			}
		}
		return false;
	}), EAllowShrinking::No);
}

void FFractureToolContext::ConvertEmbeddedSelectionToParents()
{
	Sanitize();
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;

	for (int32 SelBoneIdx = 0; SelBoneIdx < SelectedBones.Num(); ++SelBoneIdx)
	{
		int32 Bone = SelectedBones[SelBoneIdx];
		if (SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			int32 Parent = Parents[Bone];
			if (Parent != INDEX_NONE)
			{
				SelectedBones[SelBoneIdx] = Parent;
			}
			else // embedded should always have a parent, but if it somehow does not, just remove from selection
			{
				SelectedBones.RemoveAtSwap(SelBoneIdx, EAllowShrinking::No);
				--SelBoneIdx; // reconsider swapped-in-element at this idx next iter
			}
		}
	}
}

void FFractureToolContext::ConvertSelectionToRigidNodes()
{
	Sanitize();

	TArray<int32> RigidSelection;
	for (int32 Index : SelectedBones)
	{
		ConvertSelectionToRigidNodes(Index, RigidSelection);
	}

	SelectedBones = RigidSelection;
}

void FFractureToolContext::ConvertSelectionToRigidNodes(int32 Index, TArray<int32>& RigidSelection)
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_Rigid)
	{
		RigidSelection.Add(Index);
	}
	else
	{
		for (int32 Child : Children[Index])
		{
			ConvertSelectionToRigidNodes(Child, RigidSelection);
		}
	}
}

void FFractureToolContext::ConvertSelectionToEmbeddedGeometryNodes()
{
	Sanitize();

	TArray<int32> EmbeddedSelection;
	for (int32 Index : SelectedBones)
	{
		ConvertSelectionToEmbeddedGeometryNodes(Index, EmbeddedSelection);
	}

	SelectedBones = EmbeddedSelection;
}

void FFractureToolContext::ConvertSelectionToEmbeddedGeometryNodes(int32 Index, TArray<int32>& EmbeddedSelection)
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_None)
	{
		EmbeddedSelection.Add(Index);
	}
	else
	{
		for (int32 Child : Children[Index])
		{
			ConvertSelectionToEmbeddedGeometryNodes(Child, EmbeddedSelection);
		}
	}
}

void FFractureToolContext::ConvertSelectionToLevel(int32 TargetLevel, bool bAllowSelectChildren, bool bAllowSelectParents)
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	const TManagedArray<int32>* Levels = GeometryCollection->FindAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	if (TargetLevel < 0 || !Levels)
	{
		return; // No levels to filter by
	}

	TSet<int32> FilteredSelection;
	const int32 NumBones = GeometryCollection->Transform.Num();
	// Note SafetyMaxLoops is just to prevent infinite loop in case of malformed data (e.g., a cycle in the Children/Parent indices)
	int32 SafetyMaxLoops = 2 * SelectedBones.Num() * NumBones;
	while (SelectedBones.Num() && SafetyMaxLoops-- > 0)
	{
		int32 BoneIdx = SelectedBones.Pop();
		if (BoneIdx < 0 || BoneIdx >= NumBones)
		{
			continue;
		}
		int32 BoneLevel = (*Levels)[BoneIdx];
		if (BoneLevel == TargetLevel)
		{
			FilteredSelection.Add(BoneIdx);
		}
		// Optionally allow the selection to expand clusters to reach the target level
		else if (bAllowSelectChildren && TargetLevel > BoneLevel && SimulationType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered)
		{
			SelectedBones.Append(GeometryCollection->Children[BoneIdx].Array());
		}
		// Optionally allow the selection to travel to parents to reach the target level
		else if (bAllowSelectParents && TargetLevel < BoneLevel)
		{
			int32 Parent = GeometryCollection->Parent[BoneIdx];
			if (Parent > -1)
			{
				SelectedBones.Add(Parent);
			}
		}
	}
	SelectedBones = FilteredSelection.Array();
}

void FFractureToolContext::ConvertSelectionToClusterNodes()
{
	// If this is a non-cluster node, select the cluster containing it instead.
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	Sanitize();

	TArray<int32> AddedClusterSelections;
	for (int32 Index : SelectedBones)
	{
		int32 AddParent = FGeometryCollection::Invalid;
		if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			AddParent = Parents[Index];
		}
		else if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			AddParent = Parents[Parents[Index]];
		}
		if (AddParent != FGeometryCollection::Invalid)
		{
			AddedClusterSelections.AddUnique(AddParent);
		}
	}
	SelectedBones.Append(AddedClusterSelections);
	
	Sanitize();
}

void FFractureToolContext::FilterSelectionBySimulationType(FGeometryCollection::ESimulationTypes KeepSimulationType)
{
	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(*GeometryCollection);
	SelectionFacade.FilterSelectionBySimulationType(SelectedBones, KeepSimulationType);
}


bool FFractureToolContext::HasSelectedAncestor(int32 Index) const
{
	const TManagedArray<int32>& Parents = GeometryCollection->GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

	if (!ensureMsgf(Index >= 0 && Index < Parents.Num(), TEXT("Invalid index in selection: %d"), Index))
	{
		return false;
	}

	int32 CurrIndex = Index;
	while (Parents[CurrIndex] != INDEX_NONE)
	{
		CurrIndex = Parents[CurrIndex];
		if (SelectedBones.Contains(CurrIndex))
		{
			return true;
		}
	}

	// We've arrived at the top of the hierarchy with no selected ancestors
	return false;
}



