// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"


class FFractureToolContext
{
public:

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	/** Generate a tool context based on the bone selection of the specified GeometryCollectionComponent. */
	FFractureToolContext(UGeometryCollectionComponent* InGeometryCollectionComponent);

	/** @param bFavorParents	If true, when a child and its (grand)parent are both selected, only include the top-level bone in the sanitized selection */
	void Sanitize(bool bFavorParents = true);
	void ConvertSelectionToLeafNodes();
	void ConvertSelectionToRigidNodes();
	void ConvertSelectionToEmbeddedGeometryNodes();
	void ConvertSelectionToClusterNodes();
	void FilterSelectionBySimulationType(FGeometryCollection::ESimulationTypes KeepSimulationType);
	void ConvertSelectionToLevel(int32 TargetLevel, bool bAllowSelectChildren, bool bAllowSelectParents);
	void RemoveChildrenOfSelectedNodesFromSelection();
	void ConvertEmbeddedSelectionToParents();
	void RemoveRootNodes();
	/// @param ProbToKeep The chance to keep each bone in the selection
	void RandomReduceSelection(float ProbToKeep);
	
	const TArray<int32>& GetSelection() const { return SelectedBones; }
	TArray<int32>& GetSelection() { return SelectedBones; }
	void SetSelection(const TArray<int32>& NewSelection)
	{
		SelectedBones = NewSelection;
		// Note: currently Sanitize is called without favoring parents, so it removes invalid bones and sorts the selection
		// but does not prevent selection of children + parents together, in case such a selection is useful
		Sanitize(/*bFavorParents*/false);
	}
	UGeometryCollectionComponent* GetGeometryCollectionComponent() const { return GeometryCollectionComponent; }
	FGeometryCollectionPtr GetGeometryCollection() const { return GeometryCollection; }
	UGeometryCollection* GetFracturedGeometryCollection() const { return FracturedGeometryCollection; }

	/** Return selection in sibling groups */
	TMap<int32, TArray<int32>> GetClusteredSelections();

	FBox GetWorldBounds() const { return Bounds.TransformBy(Transform); }
	FBox GetBounds() const { return Bounds; }
	void SetBounds(FBox& InBounds) { Bounds = InBounds; }
	int32 GetSeed() const { return RandomSeed; }
	void SetSeed(int32 InSeed) { RandomSeed = InSeed; }
	FTransform GetTransform() const { return Transform; }

	bool IsValid() const { return GeometryCollection.IsValid() && (SelectedBones.Num()>0); }

	void GenerateGuids(int32 StartIdx);

private:
	bool HasSelectedAncestor(int32 Index) const;
	bool IsValidBone(int32 Index) const
	{
		return Index >= 0 && Index < GeometryCollection->Parent.Num();
	}
	void ConvertSelectionToLeafNodes(int32 Index, TArray<int32>& LeafSelection);
	void ConvertSelectionToRigidNodes(int32 Index, TArray<int32>& RigidSelection);
	void ConvertSelectionToEmbeddedGeometryNodes(int32 Index, TArray<int32>& EmbeddedSelection);

private:
	TArray<int32> SelectedBones;
	FGeometryCollectionPtr GeometryCollection;
	UGeometryCollectionComponent* GeometryCollectionComponent;
	UGeometryCollection* FracturedGeometryCollection;

	// Fracture specific members
	FTransform Transform;
	FBox Bounds;
	int32 RandomSeed;

};