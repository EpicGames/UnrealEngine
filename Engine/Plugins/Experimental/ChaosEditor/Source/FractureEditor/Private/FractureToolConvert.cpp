// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolConvert.h"

#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
// for content-browser things
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Components/StaticMeshComponent.h"
#include "StaticMeshAttributes.h"
#include "Engine/CollisionProfile.h"
#include "PhysicsEngine/BodySetup.h"

#include "Editor.h" // for GEditor

#include "FileHelpers.h" // TODO: rm if we aren't keeping the prompt for checkout and save fn

#include "PlanarCut.h"

#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolConvert)

#define LOCTEXT_NAMESPACE "FractureToolConvert"


UFractureToolConvert::UFractureToolConvert(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	ConvertSettings = NewObject<UFractureConvertSettings>(GetTransientPackage(), UFractureConvertSettings::StaticClass());
	ConvertSettings->OwnerTool = this;
	ReimportSettings = NewObject<UFractureReimportSettings>(GetTransientPackage(), UFractureReimportSettings::StaticClass());
	ReimportSettings->OwnerTool = this;
}

bool UFractureToolConvert::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolConvert::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolConvert", "Convert to Static Mesh"));
}

FText UFractureToolConvert::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolConvertTooltip", "This converts fracture geometry to static meshes."));
}

FSlateIcon UFractureToolConvert::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.ToMesh");
}

void UFractureToolConvert::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "ToMesh", "ToMesh", "Convert selected geometry to static meshes.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->ConvertToMesh = UICommandInfo;
}

TArray<UObject*> UFractureToolConvert::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ConvertSettings);
	Settings.Add(ReimportSettings);
	return Settings;
}

void UFractureToolConvert::FractureContextChanged()
{
}

void UFractureToolConvert::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
}

void UFractureToolConvert::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// update any cached data 
}

TArray<FFractureToolContext> UFractureToolConvert::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component, or for each individual bone if Group Fracture is not used.
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			// Generate a context for each selected node
			FFractureToolContext FullSelection(GeometryCollectionComponent);
			FullSelection.ConvertSelectionToRigidNodes();

			// Update global transforms and bounds -- TODO: pull this bounds update out to a shared function?
			const TManagedArray<FTransform3f>& Transform = FullSelection.GetGeometryCollection()->Transform;
			const TManagedArray<int32>& TransformToGeometryIndex = FullSelection.GetGeometryCollection()->TransformToGeometryIndex;
			const TManagedArray<FBox>& BoundingBoxes = FullSelection.GetGeometryCollection()->BoundingBox;

			TArray<FTransform> Transforms;
			GeometryCollectionAlgo::GlobalMatrices(Transform, FullSelection.GetGeometryCollection()->Parent, Transforms);

			FBox Bounds(ForceInit);
			for (int32 BoneIndex : FullSelection.GetSelection())
			{
				int32 GeometryIndex = TransformToGeometryIndex[BoneIndex];
				if (GeometryIndex > INDEX_NONE)
				{
					FBox BoneBound = BoundingBoxes[GeometryIndex].TransformBy(Transforms[BoneIndex]);
					Bounds += BoneBound;
				}
			}
			FullSelection.SetBounds(Bounds);

			Contexts.Add(FullSelection);
		}
	}

	return Contexts;
}

namespace UE
{
namespace FractureToolConvertInternal
{
// Create + position a single static mesh asset in the provided asset package
void CreateMeshAsset(
	FGeometryCollection& Collection, const TManagedArray<FTransform>& BoneTransforms, const FTransform& CollectionToWorld, const TConstArrayView<UMaterialInterface*>& Materials,
	TConstArrayView<int32> Bones, UPackage* AssetPackage, FString UniqueAssetName, bool bCenterPivot, bool bPlaceInWorld, bool bSelectNewActors, bool bOddMaterialsAreInternal)
{
	// create new UStaticMesh object
	EObjectFlags Flags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
	UStaticMesh* NewStaticMesh = NewObject<UStaticMesh>(AssetPackage, *UniqueAssetName, Flags);

	// initialize the LOD 0 MeshDescription
	NewStaticMesh->SetNumSourceModels(1);
	// normals and tangents should carry over from the geometry collection
	NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeNormals = false;
	NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeTangents = false;
	NewStaticMesh->GetSourceModel(0).BuildSettings.bGenerateLightmapUVs = false;

	FMeshDescription* OutputMeshDescription = NewStaticMesh->CreateMeshDescription(0);

	NewStaticMesh->CreateBodySetup();
	NewStaticMesh->GetBodySetup()->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseComplexAsSimple;

	TFunction<int32(int32, bool)> RemapMaterialIDs = nullptr;
	if (bOddMaterialsAreInternal)
	{
		RemapMaterialIDs = [](int32 InMatID, bool bIsInternal)
		{
			return InMatID * 2 + (int32)bIsInternal;
		};
	}

	FTransform BonesToCollection;
	ConvertToMeshDescription(*OutputMeshDescription, BonesToCollection, bCenterPivot, Collection, BoneTransforms, Bones, RemapMaterialIDs);

	// add a material slot. Must always have one material slot.
	int AddMaterialCount = FMath::Max(1, Materials.Num());
	for (int MatIdx = 0; MatIdx < AddMaterialCount; MatIdx++)
	{
		NewStaticMesh->GetStaticMaterials().Add(FStaticMaterial());
	}
	for (int MatIdx = 0; MatIdx < Materials.Num(); MatIdx++)
	{
		NewStaticMesh->SetMaterial(MatIdx, Materials[MatIdx]);
	}

	NewStaticMesh->CommitMeshDescription(0);

	NewStaticMesh->GetBodySetup()->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseComplexAsSimple;


	if (bPlaceInWorld)
	{
		UWorld* TargetWorld = GEditor->GetEditorWorldContext().World();
		check(TargetWorld);

		// create new actor
		FRotator Rotation(0.0f, 0.0f, 0.0f);
		FActorSpawnParameters SpawnInfo;
		AStaticMeshActor* NewActor = TargetWorld->SpawnActor<AStaticMeshActor>(FVector::ZeroVector, Rotation, SpawnInfo);
		NewActor->SetActorLabel(*UniqueAssetName);
		NewActor->GetStaticMeshComponent()->SetStaticMesh(NewStaticMesh);

		// if we don't do this, world traces don't hit the mesh
		NewActor->GetStaticMeshComponent()->RecreatePhysicsState();

		NewActor->GetStaticMeshComponent()->SetWorldTransform(CollectionToWorld * BonesToCollection);

		for (int MatIdx = 0; MatIdx < Materials.Num(); MatIdx++)
		{
			NewActor->GetStaticMeshComponent()->SetMaterial(MatIdx, Materials[MatIdx]);
		}

		NewActor->MarkComponentsRenderStateDirty();

		if (bSelectNewActors)
		{
			GEditor->SelectActor(NewActor, true, false, true, false);
		}
	}

	NewStaticMesh->PostEditChange();

	AssetPackage->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(AssetPackage);
}

// Convert and save all the requested static mesh assets
bool ConvertAndSaveMeshes(
	FGeometryCollection& Collection, const TManagedArray<FTransform>& BoneTransforms, const FTransform& CollectionToWorld,
	const TConstArrayView<UMaterialInterface*>& Materials, TConstArrayView<int32> Bones, FString ObjectBaseName, 
	const UObject* RelativeToAsset, bool bPromptToSave, bool bSaveCombined, bool bCenterPivots, bool bPlaceInWorld,
	bool bSelectNewActors, bool bOddMaterialsAreInternal)
{
	check(RelativeToAsset);

	// find path to reference asset
	UPackage* AssetOuterPackage = CastChecked<UPackage>(RelativeToAsset->GetOuter());
	FString AssetPackageName = AssetOuterPackage->GetName();
	FString PackageFolderPath = FPackageName::GetLongPackagePath(AssetPackageName);

	// Show the modal dialog and then get the path/name.
	// If the user cancels, conversion is skipped
	if (bPromptToSave)
	{
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

		FString UseDefaultAssetName = ObjectBaseName;
		FString CurrentPath = PackageFolderPath;

		FSaveAssetDialogConfig Config;
		Config.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		Config.DefaultAssetName = UseDefaultAssetName;
		Config.DialogTitleOverride = bSaveCombined ? 
			LOCTEXT("GenerateBoneMeshAssetPathDialogWarningCombine", "Choose Folder Path and Base Name for Converted Mesh.") :
			LOCTEXT("GenerateBoneMeshAssetPathDialogWarningPerBone", "Choose Folder Path and Base Name for Converted Meshes. Bone index will be appended to each mesh.");
		Config.DefaultPath = CurrentPath;
		FString SelectedPath = ContentBrowser.CreateModalSaveAssetDialog(Config);

		if (SelectedPath.IsEmpty() == false)
		{
			PackageFolderPath = FPaths::GetPath(SelectedPath);
			ObjectBaseName = FPaths::GetBaseFilename(SelectedPath, true);
		}
		else
		{
			return false;
		}
	}

	auto MakeUniquePackage = [&PackageFolderPath](FString BaseName, FString& UniqueAssetNameOut) -> UPackage*
	{
		// create new package
		FString UniquePackageName;

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(PackageFolderPath + TEXT("/") + BaseName, TEXT(""), UniquePackageName, UniqueAssetNameOut);
		return CreatePackage(*UniquePackageName);
	};

	FScopedSlowTask ConvertTask(Bones.Num(), LOCTEXT("StartingConvert", "Converting geometry to static mesh"));
	ConvertTask.MakeDialog();

	if (bSelectNewActors)
	{
		GEditor->SelectNone(false, true, false);
	}

	TArray<UPackage*> SavePackage;
	if (bSaveCombined)
	{
		ConvertTask.EnterProgressFrame(Bones.Num());
		FString UniqueAssetName;
		UPackage* AssetPackage = MakeUniquePackage(ObjectBaseName, UniqueAssetName);
		SavePackage.Add(AssetPackage);
		CreateMeshAsset(Collection, BoneTransforms, CollectionToWorld, Materials, Bones, AssetPackage, UniqueAssetName, bCenterPivots, bPlaceInWorld, bSelectNewActors, bOddMaterialsAreInternal);
	}
	else
	{
		for (int SelIdx = 0; SelIdx < Bones.Num(); SelIdx++)
		{
			ConvertTask.EnterProgressFrame(1);

			int32 UseBone = Bones[SelIdx];

			FString UseBaseName = FString::Printf(TEXT("%s_%d_"), *ObjectBaseName, UseBone);

			FString UniqueAssetName;
			UPackage* AssetPackage = MakeUniquePackage(UseBaseName, UniqueAssetName);
			SavePackage.Add(AssetPackage);

			TArrayView<const int32> SingleBoneView(&UseBone, 1);

			CreateMeshAsset(Collection, BoneTransforms, CollectionToWorld, Materials, SingleBoneView, AssetPackage, UniqueAssetName, bCenterPivots, bPlaceInWorld, bSelectNewActors, bOddMaterialsAreInternal);
		}
	}

	if (bSelectNewActors)
	{
		GEditor->NoteSelectionChange(true);
	}

	FEditorFileUtils::PromptForCheckoutAndSave(SavePackage, true, true);

	return true;
}
}} // namespace UE::FractureToolConvertInternal


int32 UFractureToolConvert::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		const TManagedArray<FTransform>& BoneTransforms = FractureContext.GetGeometryCollectionComponent()->RestTransforms;

		FMeshDescription Mesh;
		FTransform MeshTransform;

		TArray<TObjectPtr<class UMaterialInterface>>& OverrideMaterials = FractureContext.GetGeometryCollectionComponent()->OverrideMaterials;
		TArray<TObjectPtr<class UMaterialInterface>>& MainMaterials = FractureContext.GetFracturedGeometryCollection()->Materials;

		int32 LastMaterialIdx = MainMaterials.Num() - 1;
		// if possible, we'd like to skip the last material -- it should be the selection material, which should be unused
		int32 SkipLastMaterialOffset = 1;
		if (LastMaterialIdx >= 0)
		{
			for (int32 UsedMaterialID : Collection.MaterialID)
			{
				if (UsedMaterialID == LastMaterialIdx)
				{
					// last material is used in the mesh; we can't skip it
					SkipLastMaterialOffset = 0;
					break;
				}
			}
		}

		TArray<UMaterialInterface*> Materials;
		Materials.Reserve(MainMaterials.Num());
		for (int32 MatIdx = 0; MatIdx + SkipLastMaterialOffset < MainMaterials.Num(); MatIdx++)
		{
			if (MatIdx < OverrideMaterials.Num() && OverrideMaterials[MatIdx])
			{
				Materials.Add(OverrideMaterials[MatIdx].Get());
			}
			else
			{
				Materials.Add(MainMaterials[MatIdx].Get());
			}
		}

		// If we need odd materials as internal, naively duplicate materials
		// Note: We could do this in a smarter pattern based to minimize material slots, but at the cost of having a less predictable / more confusing result
		if (ReimportSettings->bOddMaterialsAreInternal)
		{
			int32 RealMaterialNum = Materials.Num();
			Materials.SetNum(RealMaterialNum * 2);
			for (int32 MaterialIndex = RealMaterialNum - 1; MaterialIndex >= 0; --MaterialIndex)
			{
				Materials[MaterialIndex * 2 + 1] = Materials[MaterialIndex];
				Materials[MaterialIndex * 2] = Materials[MaterialIndex];
			}
		}

		// choose default mesh name based on corresponding geometry collection name
		FString BaseName = FString::Printf(TEXT("%s_SM"), *FractureContext.GetFracturedGeometryCollection()->GetName());
		
		UE::FractureToolConvertInternal::ConvertAndSaveMeshes(Collection, BoneTransforms, FractureContext.GetTransform(), Materials,
			FractureContext.GetSelection(), BaseName, FractureContext.GetFracturedGeometryCollection(), 
			ConvertSettings->bPromptForBaseName, !ConvertSettings->bPerBone, ConvertSettings->bCenterPivots, ConvertSettings->bPlaceInWorld, ConvertSettings->bSelectNewActors, ReimportSettings->bOddMaterialsAreInternal);
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE

