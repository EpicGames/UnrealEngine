// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolConvert.generated.h"

class FFractureToolContext;
struct FMeshDescription;

/** Settings related to geometry collection -> static mesh conversion **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureConvertSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureConvertSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Whether to prompt user for a location and base name for the generated meshes, or automatically place them next to the source geometry collections */
	UPROPERTY(EditAnywhere, Category = AssetSettings)
	bool bPromptForBaseName = true;

	/** Whether to generate a separate mesh for each bone, or one combined mesh */
	UPROPERTY(EditAnywhere, Category = AssetSettings)
	bool bPerBone = true;

	/** Whether to center the pivot for each mesh, or use the pivot from the geometry collection */
	UPROPERTY(EditAnywhere, Category = GeometrySettings)
	bool bCenterPivots = false;

	/** Whether to place new static mesh actors in the level */
	UPROPERTY(EditAnywhere, Category = ActorSettings)
	bool bPlaceInWorld = true;

	/** Whether to select new static mesh actors */
	UPROPERTY(EditAnywhere, Category = ActorSettings, meta = (EditCondition = "bPlaceInWorld"))
	bool bSelectNewActors = false;
};

/** Settings related to geometry collection -> static mesh conversion **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureReimportSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	// If checked, materials will be updated on export to ensure that odd-numbered material IDs correspond to internal faces, and this will be used on re-import to assign correct material IDs
	UPROPERTY(EditAnywhere, Category = ReimportSettings)
	bool bOddMaterialsAreInternal = true;

	// TODO: Move the "re-import ToMesh output" path from out of the Generate/New UI an into this tool.  (Or provide alternate tooling and deprecate the need for this path.)
	// Reimport a static mesh to the currently-selected geometry collection, or to a new geometry collection if none is selected.
	//UFUNCTION(CallInEditor, Category = ReimportSettings, meta = (DisplayName = "Reimport to Selected"))
	//void ReimportToSelected();
};

UCLASS(DisplayName = "Convert Tool", Category = "FractureTools")
class UFractureToolConvert : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolConvert(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureToolConvert", "ExecuteConvert", "Convert")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;
	virtual bool ExecuteUpdatesShape() const override
	{
		return false;
	}
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Gets the UI command info for this command */
	const TSharedPtr<FUICommandInfo>& GetUICommandInfo() const;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;


protected:
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureConvertSettings> ConvertSettings;

	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureReimportSettings> ReimportSettings;
};
