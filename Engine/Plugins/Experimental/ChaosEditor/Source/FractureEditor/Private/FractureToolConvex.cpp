// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolConvex.h"

#include "SGeometryCollectionOutliner.h"
#include "FractureToolContext.h"
#include "FractureModeSettings.h"
#include "FractureSettings.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolConvex)

#define LOCTEXT_NAMESPACE "FractureToolConvex"


void UFractureConvexSettings::DeleteFromSelected()
{
	UFractureToolConvex* ConvexTool = Cast<UFractureToolConvex>(OwnerTool.Get());
	ConvexTool->DeleteConvexFromSelected();
}

// Note: this feature puts multiple convexes on a single bone, which isn't supported by sim yet
#if 0
void UFractureConvexSettings::PromoteChildren()
{
	UFractureToolConvex* ConvexTool = Cast<UFractureToolConvex>(OwnerTool.Get());
	ConvexTool->PromoteChildren();
}
#endif

void UFractureConvexSettings::ClearCustomConvex()
{
	UFractureToolConvex* ConvexTool = Cast<UFractureToolConvex>(OwnerTool.Get());
	ConvexTool->ClearCustomConvex();
}

void UFractureConvexActions::SaveAsDefaults()
{
	UFractureToolConvex* ConvexTool = Cast<UFractureToolConvex>(OwnerTool.Get());
	UFractureModeSettings* ModeSettings = GetMutableDefault<UFractureModeSettings>();
	ModeSettings->Modify();
	ModeSettings->ConvexFractionAllowRemove = ConvexTool->ConvexSettings->FractionAllowRemove;
	ModeSettings->ConvexCanExceedFraction = ConvexTool->ConvexSettings->CanExceedFraction;
	ModeSettings->ConvexSimplificationDistanceThreshold = ConvexTool->ConvexSettings->SimplificationDistanceThreshold;
	ModeSettings->ConvexRemoveOverlaps = ConvexTool->ConvexSettings->RemoveOverlaps;
	ModeSettings->ConvexOverlapRemovalShrinkPercent = ConvexTool->ConvexSettings->OverlapRemovalShrinkPercent;
}

void UFractureConvexActions::SetFromDefaults()
{
	UFractureToolConvex* ConvexTool = Cast<UFractureToolConvex>(OwnerTool.Get());
	const UFractureModeSettings* ModeSettings = GetDefault<UFractureModeSettings>();
	ConvexTool->ConvexSettings->FractionAllowRemove = ModeSettings->ConvexFractionAllowRemove;
	ConvexTool->ConvexSettings->CanExceedFraction = ModeSettings->ConvexCanExceedFraction;
	ConvexTool->ConvexSettings->SimplificationDistanceThreshold = ModeSettings->ConvexSimplificationDistanceThreshold;
	ConvexTool->ConvexSettings->RemoveOverlaps = ModeSettings->ConvexRemoveOverlaps;
	ConvexTool->ConvexSettings->OverlapRemovalShrinkPercent = ModeSettings->ConvexOverlapRemovalShrinkPercent;

	ConvexTool->NotifyOfPropertyChangeByTool(this);
}

void UFractureToolConvex::DeleteConvexFromSelected()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

		if (!Collection.HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup) ||
			!Collection.HasAttribute("TransformToConvexIndices", FGeometryCollection::TransformGroup))
		{
			continue;
		}

		// Note: Must make sure the custom flags attribute is added *before* the CollectionEdit calls Modify() on the collection, because serialization is weird for
		// the geometry collection: undo will load the saved attributes back, but not clear the attributes that didn't exist before
		// TODO: Consider making the serialization clear the attributes that didn't exist before, then move this line after the CollectionEdit line
		TManagedArray<int32>& HasCustomConvex = *FGeometryCollectionConvexUtility::GetCustomConvexFlags(&Collection, true);

		FGeometryCollectionEdit CollectionEdit(FractureContext.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::Rest, true /*bShapeIsUnchanged -- use to skip re-generating convexes, because we directly update them */);

		TArray<int32> TransformsToClear;
		for (int32 TransformIdx : FractureContext.GetSelection())
		{
			if (Collection.SimulationType[TransformIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				TransformsToClear.Add(TransformIdx);
				HasCustomConvex[TransformIdx] = 1;
			}
		}
		TransformsToClear.Sort();
		FGeometryCollectionConvexUtility::RemoveConvexHulls(&Collection, TransformsToClear);
	}

	FractureContextChanged();
}

void UFractureToolConvex::PromoteChildren()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

		if (!Collection.HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup) ||
			!Collection.HasAttribute("TransformToConvexIndices", FGeometryCollection::TransformGroup))
		{
			continue;
		}
		
		// Note: Must make sure the custom flags attribute is added *before* the CollectionEdit calls Modify() on the collection, because serialization is weird for
		// the geometry collection: undo will load the saved attributes back, but not clear the attributes that didn't exist before
		// TODO: Consider making the serialization clear the attributes that didn't exist before, then move this line after the CollectionEdit line
		FGeometryCollectionConvexUtility::GetCustomConvexFlags(FractureContext.GetGeometryCollection().Get(), true);

		FGeometryCollectionEdit CollectionEdit(FractureContext.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::Rest, true /*bShapeIsUnchanged -- use to skip re-generating convexes, because we directly update them */);

		TArray<int32> SelectedTransforms = FractureContext.GetSelection();
		FGeometryCollectionConvexUtility::CopyChildConvexes(&Collection, SelectedTransforms, &Collection, SelectedTransforms, false);
	}

	FractureContextChanged();
}

void UFractureToolConvex::ClearCustomConvex()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	bool bAnyChanged = false;
	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		bool bHasChanged = false;

		TManagedArray<int32>* HasCustomConvex = FGeometryCollectionConvexUtility::GetCustomConvexFlags(FractureContext.GetGeometryCollection().Get(), false);
		if (!HasCustomConvex)
		{
			continue;
		}

		FGeometryCollectionEdit CollectionEdit(FractureContext.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::Rest);

		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		for (int32 TransformIdx : FractureContext.GetSelection())
		{
			if ((*HasCustomConvex)[TransformIdx])
			{
				bAnyChanged = bHasChanged = true;
				(*HasCustomConvex)[TransformIdx] = false;
			}
		}

		if (bHasChanged)
		{
			bool bAllFalse = true;
			for (int32 TransformIdx = 0; bAllFalse && TransformIdx < HasCustomConvex->Num(); TransformIdx++)
			{
				if ((*HasCustomConvex)[TransformIdx])
				{
					bAllFalse = false;
				}
			}
			if (bAllFalse)
			{
				Collection.RemoveAttribute("HasCustomConvex", FTransformCollection::TransformGroup);
			}
		}
	}

	if (bAnyChanged)
	{
		FractureContextChanged();
	}
}

UFractureToolConvex::UFractureToolConvex(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	ConvexSettings = NewObject<UFractureConvexSettings>(GetTransientPackage(), UFractureConvexSettings::StaticClass());
	ConvexSettings->OwnerTool = this;
	ConvexActions = NewObject<UFractureConvexActions>(GetTransientPackage(), UFractureConvexActions::StaticClass());
	ConvexActions->OwnerTool = this;
}

bool UFractureToolConvex::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolConvex::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolConvex", "Update Convex Collision Volumes"));
}

FText UFractureToolConvex::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolConvexTooltip", "This tool visualizes and updates settings for convex hull generation on the geometry collections"));
}

FSlateIcon UFractureToolConvex::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Convex");
}

void UFractureToolConvex::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Convex", "Convex", "Update (and visualize) a hierarchy of non-overlapping convex collision volumes for the bones of geometry collections.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->MakeConvex = UICommandInfo;
}

TArray<UObject*> UFractureToolConvex::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ConvexSettings);
	Settings.Add(ConvexActions);
	return Settings;
}

void UFractureToolConvex::AddConvexHullVizualizationData(const Chaos::FConvex& ConvexHull, const FTransform& WorldSpaceTransform, bool bIsCustom, bool bIsUnionOfHulls)
{
	int32 HullPtsStart = HullPoints.Num();
	for (Chaos::FVec3 Pt : ConvexHull.GetVertices())
	{
		HullPoints.Add(WorldSpaceTransform.TransformPosition(Pt));
	}
	int32 NumPlanes = ConvexHull.NumPlanes();
	const Chaos::FConvexStructureData& HullData = ConvexHull.GetStructureData();
	for (int PlaneIdx = 0; PlaneIdx < NumPlanes; PlaneIdx++)
	{
		int32 NumPlaneVerts = HullData.NumPlaneVertices(PlaneIdx);
		for (int32 PlaneVertexIdx = 0; PlaneVertexIdx < NumPlaneVerts; PlaneVertexIdx++)
		{
			int32 V0 = HullPtsStart + HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx);
			int32 V1 = HullPtsStart + HullData.GetPlaneVertex(PlaneIdx, (PlaneVertexIdx + 1) % NumPlaneVerts);
			HullEdges.Add({ V0, V1, bIsCustom, bIsUnionOfHulls });
		}
	}
}

void UFractureToolConvex::AddVisualizationData(FGeometryCollection& Collection, const FTransform& OuterTransform, int32 TransformIdx, bool bUseExternalCollision, bool bIsUnionOfHulls)
{
	const TManagedArray<int32>* HasCustomConvex = FGeometryCollectionConvexUtility::GetCustomConvexFlags(&Collection, false);
	const TManagedArray<TSet<int32>>* TransformToConvexIndices = Collection.FindAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	const TManagedArray<Chaos::FConvexPtr>* ConvexHull = Collection.FindAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	const TManagedArray<Chaos::FImplicitObjectPtr>* ExternalCollisions = Collection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

	const FTransform ComponentSpaceTransform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TransformIdx);
	const FTransform WorldSpaceTransform = ComponentSpaceTransform * OuterTransform;

	const bool bHasAnyConvex = TransformToConvexIndices && ConvexHull && ((*TransformToConvexIndices)[TransformIdx].Num() > 0);
	const bool bIsCustom = bHasAnyConvex && (HasCustomConvex ? bool((*HasCustomConvex)[TransformIdx]) : false);
	const bool bHasExternalCollision = ExternalCollisions && ((*ExternalCollisions)[TransformIdx].IsValid());

	// order of priority : external collision / generated convex / union of children shapes
	if (bHasExternalCollision && bUseExternalCollision)
	{
		TArray<FGeometryCollectionConvexUtility::FTransformedConvex> ExternalHulls;
		FGeometryCollectionConvexUtility::ConvertImplicitToConvexArray(*(*ExternalCollisions)[TransformIdx], WorldSpaceTransform, ExternalHulls);
		for (const FGeometryCollectionConvexUtility::FTransformedConvex& ExternalHull : ExternalHulls)
		{
			AddConvexHullVizualizationData(*ExternalHull.Convex, ExternalHull.Transform, bIsCustom, bIsUnionOfHulls);
		}
	}
	else if (bHasAnyConvex)
	{
		for (int32 ConvexIdx : (*TransformToConvexIndices)[TransformIdx])
		{
			AddConvexHullVizualizationData(*(*ConvexHull)[ConvexIdx], WorldSpaceTransform, bIsCustom, bIsUnionOfHulls);
		}
	}
	else
	{
		// no convex generated, we are using a union of the children convexes
		for (int32 ChildTransformIndex : Collection.Children[TransformIdx])
		{
			AddVisualizationData(Collection, OuterTransform, ChildTransformIndex, bUseExternalCollision, /* bIsUnionOfHulls */ true);
		}
	}
}

void UFractureToolConvex::FractureContextChanged()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>();

	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		if (UGeometryCollection* RestCollection = FractureContext.GetFracturedGeometryCollection())
		{
			FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

			const int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());

			//FTransform OuterTransform = FractureContext.GetTransform();
			const FTransform OuterTransform = FractureContext.GetGeometryCollectionComponent()->GetComponentTransform();

			FractureContext.ConvertSelectionToLevel(FractureSettings->FractureLevel, 
				true /*Auto-select children at the target level*/, false /*Don't auto-select parents*/);

			for (int32 TransformIdx : FractureContext.GetSelection())
			{
				EdgesMappings.AddMapping(CollectionIdx, TransformIdx, HullEdges.Num());

				AddVisualizationData(Collection, OuterTransform, TransformIdx, RestCollection->bImportCollisionFromSource);
			}
		}
	}
}

void UFractureToolConvex::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	const bool bCaptureSeeThroughLines = ConvexSettings->bSeeThroughLines;
	const float CaptureLineThickness = ConvexSettings->LineThickness;
	EnumerateVisualizationMapping(EdgesMappings, HullEdges.Num(), [&](int32 Idx, FVector ExplodedVector)
	{
		const FEdgeVisInfo& Edge = HullEdges[Idx];
		FVector P1 = HullPoints[Edge.A] + ExplodedVector;
		FVector P2 = HullPoints[Edge.B] + ExplodedVector;
		FLinearColor Color = FLinearColor::Blue;
		if (Edge.bIsCustom) { Color = FLinearColor(1, 0, 1); /* purple */ }
		if (Edge.bIsUnion)	{ Color = FLinearColor::Red;  }
		PDI->DrawLine(P1, P2, Color, bCaptureSeeThroughLines? SDPG_Foreground: SDPG_World, CaptureLineThickness, 0.005f);
	});
}

void UFractureToolConvex::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// update any cached data 
}

TArray<FFractureToolContext> UFractureToolConvex::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component, or for each individual bone if Group Fracture is not used.
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		// Generate a context for each selected node
		FFractureToolContext FullSelection(GeometryCollectionComponent);

		Contexts.Add(FullSelection);
	}

	return Contexts;
}

int32 UFractureToolConvex::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.GetGeometryCollection().IsValid())
	{
		FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties Properties = FractureContext.GetGeometryCollection()->GetConvexProperties();
		Properties.FractionRemove = ConvexSettings->FractionAllowRemove;
		Properties.SimplificationThreshold = ConvexSettings->SimplificationDistanceThreshold;
		Properties.CanExceedFraction = ConvexSettings->CanExceedFraction;
		Properties.RemoveOverlaps = ConvexSettings->RemoveOverlaps;
		Properties.OverlapRemovalShrinkPercent = ConvexSettings->OverlapRemovalShrinkPercent;
		FractureContext.GetGeometryCollection()->SetConvexProperties(Properties);
	}

	return INDEX_NONE;
}

void UFractureToolConvex::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	if (InToolkit.IsValid())
	{
		InToolkit.Pin()->SetOutlinerColumnMode(EOutlinerColumnMode::Collision);
	}
}

#undef LOCTEXT_NAMESPACE

