// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "GeometryCollection/GeometryCollectionConvexUtility.h"

#include "FractureToolConvex.generated.h"

class FFractureToolContext;

/** Settings controlling how convex hulls are generated for geometry collections */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureConvexSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureConvexSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Fraction (of geometry volume) by which a cluster's convex hull volume can exceed the actual geometry volume before instead using the hulls of the children.  0 means the convex volume cannot exceed the geometry volume; 1 means the convex volume is allowed to be 100% larger (2x) the geometry volume. */
	UPROPERTY(EditAnywhere, Category = Automatic, meta = (DisplayName = "Allow Larger Hull Fraction", ClampMin = "0"))
	double CanExceedFraction = .5;

	/** We simplify the convex shape to keep points spaced at least this far apart (except to keep the hull from collapsing to zero volume) */
	UPROPERTY(EditAnywhere, Category = Automatic, meta = (ClampMin = "0"))
	double SimplificationDistanceThreshold = 10.0;

	/** Whether to automatically cut away overlapping parts of the convex hulls, to avoid the simulation 'popping' to fix the overlaps */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval)
	EConvexOverlapRemoval RemoveOverlaps = EConvexOverlapRemoval::All;

	/** Overlap removal will be computed as if convex hulls were this percentage smaller (in range 0-100) */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval, meta = (UIMin = "0", ClampMax = "99.9"))
	double OverlapRemovalShrinkPercent = 0.0;

	/** Fraction of the convex hulls for a cluster that we can remove before using the hulls of the children */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval, meta = (DisplayName = "Max Removal Fraction", ClampMin = ".01", ClampMax = "1"))
	double FractionAllowRemove = .5;

	/** Delete convex hulls from selected clusters.  Does not affect hulls on leaves. */
	UFUNCTION(CallInEditor, Category = Custom, meta = (DisplayName = "Delete From Selected"))
	void DeleteFromSelected();

	/** When enabled, convex visualization lines will show through the actual geometry */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "See Through Lines"))
	bool bSeeThroughLines = false;

	/** line thickness*/
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Line Thickness", ClampMin = ".00", ClampMax = "1.0"))
	float LineThickness = 0.5f;

	// Note: this feature puts multiple convexes on a single bone, which isn't supported by sim yet
#if 0
	/** Promote (and save) child convex hulls on to the selected bone(s) */
	UFUNCTION(CallInEditor, Category = Custom, meta = (DisplayName = "Promote Children"))
	void PromoteChildren();
#endif

	/** Clear any manual adjustments to convex hulls on the selected bones */
	UFUNCTION(CallInEditor, Category = Custom, meta = (DisplayName = "Clear Custom Convex"))
	void ClearCustomConvex();
};

/**
 * UFUNCTION actions to manage convex hulls generation for geometry collections
 * (These are pulled out from the above settings object mainly to control their ordering in the properties panel)
 */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureConvexActions : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureConvexActions(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Save settings as project defaults, to be used for all new geometry collections */
	UFUNCTION(CallInEditor, Category = ProjectDefaults, meta = (DisplayName = "Save As Defaults"))
	void SaveAsDefaults();

	/** Set settings from current project defaults */
	UFUNCTION(CallInEditor, Category = ProjectDefaults, meta = (DisplayName = "Set From Defaults"))
	void SetFromDefaults();


};


UCLASS(DisplayName = "Convex Tool", Category = "FractureTools")
class UFractureToolConvex : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolConvex(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("Convex", "ExecuteConvex", "Update Convex Hulls")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Gets the UI command info for this command */
	const TSharedPtr<FUICommandInfo>& GetUICommandInfo() const;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	void DeleteConvexFromSelected();
	void PromoteChildren();
	void ClearCustomConvex();

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;


	UPROPERTY(EditAnywhere, Category = Convex)
	TObjectPtr<UFractureConvexSettings> ConvexSettings;

protected:

	UPROPERTY(EditAnywhere, Category = Convex)
	TObjectPtr<UFractureConvexActions> ConvexActions;

	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		HullPoints.Empty();
		HullEdges.Empty();
		EdgesMappings.Empty();
	}

	void AddConvexHullVizualizationData(const Chaos::FConvex& ConvexHull, const FTransform& WorldSpaceTransform, bool bIsCustom, bool bIsUnionOfHulls);
	void AddVisualizationData(FGeometryCollection& Collection, const FTransform& OuterTransform, int32 TransformIdx, bool bUseExternalCollision, bool bIsUnionOfHulls = false);

	struct FEdgeVisInfo
	{
		int32 A, B;
		int8 bIsCustom : 1; // allow for different coloring of hulls that have been manually set vs auto-generated ones
		int8 bIsUnion : 1; // this edge is part of a union of hulls ( cluster does not have a convex and is using a union of the children )
	};

	TArray<FVector> HullPoints;
	TArray<FEdgeVisInfo> HullEdges;
	FVisualizationMappings EdgesMappings;

};
