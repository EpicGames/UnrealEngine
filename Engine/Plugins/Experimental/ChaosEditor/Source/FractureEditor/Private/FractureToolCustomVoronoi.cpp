// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolCustomVoronoi.h"

#include "FractureToolContext.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "StaticMeshAttributes.h"
#include "MeshDescription.h"
#include "Spatial/PriorityOrderPoints.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolCustomVoronoi)

#define LOCTEXT_NAMESPACE "FractureCustomVoronoi"


void UFractureCustomVoronoiSettings::FreezeLiveSites()
{
	UFractureToolCustomVoronoi* CustomVoronoiTool = CastChecked<UFractureToolCustomVoronoi>(OwnerTool);
	CustomVoronoiTool->FreezeLiveSites();
	CustomVoronoiTool->FractureContextChanged();
}

void UFractureCustomVoronoiSettings::ClearFrozenSites()
{
	UFractureToolCustomVoronoi* CustomVoronoiTool = CastChecked<UFractureToolCustomVoronoi>(OwnerTool);
	CustomVoronoiTool->ClearFrozenSites();
	CustomVoronoiTool->FractureContextChanged();
}

void UFractureCustomVoronoiSettings::UnfreezeSites()
{
	UFractureToolCustomVoronoi* CustomVoronoiTool = CastChecked<UFractureToolCustomVoronoi>(OwnerTool);
	CustomVoronoiTool->UnfreezeSites();
	CustomVoronoiTool->FractureContextChanged();
}

void UFractureCustomVoronoiSettings::RegenerateLiveSites()
{
	UFractureToolCustomVoronoi* CustomVoronoiTool = CastChecked<UFractureToolCustomVoronoi>(OwnerTool);
	CustomVoronoiTool->ClearLiveSites();
	CustomVoronoiTool->FractureContextChanged();
}


void UFractureCustomVoronoiSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	RegenerateLiveSites();

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}



UFractureToolCustomVoronoi::UFractureToolCustomVoronoi(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	CustomVoronoiSettings = NewObject<UFractureCustomVoronoiSettings>(GetTransientPackage(), UFractureCustomVoronoiSettings::StaticClass());
	CustomVoronoiSettings->OwnerTool = this;
	GizmoSettings = NewObject<UFractureTransformGizmoSettings>(GetTransientPackage(), UFractureTransformGizmoSettings::StaticClass());
	GizmoSettings->OwnerTool = this;
}

void UFractureToolCustomVoronoi::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	GizmoSettings->Setup(this, ETransformGizmoSubElements::FullTranslateRotateScale);
	// Stop scaling at 0 rather than going negative
	GizmoSettings->TransformGizmo->SetDisallowNegativeScaling(true);
	// allow non uniform scale even when the gizmo mode is set to "world"
	GizmoSettings->TransformGizmo->SetIsNonUniformScaleAllowedFunction([]() {
		return true;
	});
	// Always position the points with a gizmo
	GizmoSettings->bShowUseGizmoOption = false;
}


void UFractureToolCustomVoronoi::Shutdown()
{
	Super::Shutdown();
	GizmoSettings->Shutdown();
}

FText UFractureToolCustomVoronoi::GetDisplayText() const
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolCustomVoronoi", "Custom Voronoi Fracture")); 
}

FText UFractureToolCustomVoronoi::GetTooltipText() const 
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolCustomVoronoiTooltip", "Custom Voronoi Fracture creates Voronoi cells from a customizable distribution of points, which you can transform and layer.  Click the Fracture Button to commit the fracture to the geometry collection."));
}

FSlateIcon UFractureToolCustomVoronoi::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.CustomVoronoi");
}

void UFractureToolCustomVoronoi::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "CustomVoronoi", "Custom", "Fracture with a customizable Voronoi diagram. Transform and layer arrangements of Voronoi sites to design your own fracture pattern.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->CustomVoronoi = UICommandInfo;
}

TArray<UObject*> UFractureToolCustomVoronoi::GetSettingsObjects() const 
{ 
	TArray<UObject*> Settings;
	Settings.Add(CustomVoronoiSettings);
	Settings.Add(GizmoSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

void UFractureToolCustomVoronoi::GenerateLivePattern(const TArray<FFractureToolContext>& FractureContexts, int32 RandomSeed)
{
	if (!ensure(CombinedWorldBounds.IsValid))
	{
		return;
	}

	if (!ensure(LiveSites.Num() == 0)) // We don't call this when we already have sites generated
	{
		LiveSites.Empty();
	}

	FRandomStream RandStream(RandomSeed);

	FBox& Bounds = CombinedWorldBounds; // shorter name for convenience
	TArray<FVector>& Sites = LiveSites;
	TArray<float> ImportanceWeights;
	bool bUseOrderedSkip = CustomVoronoiSettings->SkipFraction > 0 
		&& (CustomVoronoiSettings->SkipMode == EDownsamplingMode::UniformSpacing || CustomVoronoiSettings->SkipMode == EDownsamplingMode::KeepSharp);

	const FVector Extent(Bounds.Max - Bounds.Min);
	int32 NumSites = CustomVoronoiSettings->SitesToAdd;

	FTransform GizmoTransform = GetGizmoTransform();

	if (CustomVoronoiSettings->VoronoiPattern == EVoronoiPattern::Uniform)
	{
		// Uniform noise on the current bounds
		for (int32 Idx = 0; Idx < NumSites; ++Idx)
		{
			Sites.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		}
	}
	else if (CustomVoronoiSettings->VoronoiPattern == EVoronoiPattern::Centered)
	{
		// Noise centered on the gizmo
		if (CustomVoronoiSettings->Variability > 0)
		{
			for (int32 Idx = 0; Idx < NumSites; ++Idx)
			{
				Sites.Add(GizmoTransform.GetLocation());
			}
		}
		else
		{
			// if there's no variability, all sites would be at the same spot so we only need one of them
			Sites.Add(GizmoTransform.GetLocation());
		}
	}
	else if (CustomVoronoiSettings->VoronoiPattern == EVoronoiPattern::MeshVertices)
	{
		auto HasMesh = [](AStaticMeshActor* ReferenceMesh) -> bool
		{
			if (ReferenceMesh == nullptr)
			{
				return false;
			}
			const UStaticMeshComponent* Component = ReferenceMesh->GetStaticMeshComponent();
			if (Component == nullptr)
			{
				return false;
			}
			const UStaticMesh* Mesh = Component->GetStaticMesh();
			if (Mesh == nullptr)
			{
				return false;
			}
			if (Mesh->GetNumLODs() < 1)
			{
				return false;
			}
			return true;
		};
		AStaticMeshActor* ReferenceMesh = CustomVoronoiSettings->ReferenceMesh.Get();
		if (HasMesh(ReferenceMesh))
		{
			FTransform VerticesTransform = ReferenceMesh->GetStaticMeshComponent()->GetComponentTransform();
			if (!CustomVoronoiSettings->bStartAtActor)
			{
				VerticesTransform.SetLocation(GizmoTransform.GetLocation());
			}

			FMeshDescription* MeshDescription = ReferenceMesh->GetStaticMeshComponent()->GetStaticMesh()->GetMeshDescription(0);
			TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription->GetVertexPositions();
			TArray<FVector3f> Normals; // Per-vertex normals for offsets
			float NormalOffset = CustomVoronoiSettings->NormalOffset;
			if (NormalOffset > 0 || bUseOrderedSkip)
			{
				Normals.SetNumZeroed(VertexPositions.GetNumElements());

				FStaticMeshConstAttributes Attributes(*MeshDescription);
				TVertexInstanceAttributesConstRef<FVector3f> InstanceNormals = Attributes.GetVertexInstanceNormals();
				for (const FVertexInstanceID InstanceID : MeshDescription->VertexInstances().GetElementIDs())
				{
					FVector3f Normal = InstanceNormals.Get(InstanceID);
					FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(InstanceID);
					Normals[VertexID] += Normal;
				}
				for (FVector3f& Normal : Normals)
				{
					Normal.Normalize();
				}
				if (bUseOrderedSkip)
				{
					ImportanceWeights.SetNumZeroed(Normals.Num());
					for (const FVertexInstanceID InstanceID : MeshDescription->VertexInstances().GetElementIDs())
					{
						FVector3f Normal = InstanceNormals.Get(InstanceID);
						FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(InstanceID);
						ImportanceWeights[VertexID] = FMath::Max(ImportanceWeights[VertexID], 1 - Normal.Dot(Normals[VertexID]));
					}
				}
			}
			for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				FVector Position = (FVector)VertexPositions.Get(VertexID);
				if (NormalOffset > 0)
				{
					Position += (FVector)Normals[VertexID] * NormalOffset;
				}
				Sites.Add(VerticesTransform.TransformPosition(Position));
			}
		}
	}
	else if (CustomVoronoiSettings->VoronoiPattern == EVoronoiPattern::SelectedBones)
	{
		for (const FFractureToolContext& Context : FractureContexts)
		{
			FGeometryCollection& Collection = *Context.GetGeometryCollection();
			for (int32 Bone : Context.GetSelection())
			{
				FTransform BoneTransform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, Bone);
				int32 GeometryIdx = Collection.TransformToGeometryIndex[Bone];
				if (GeometryIdx == INDEX_NONE)
				{
					continue;
				}
				int32 Start = Collection.VertexStart[GeometryIdx];
				int32 Count = Collection.VertexCount[GeometryIdx];
				// for now, always exclude vertices that are not used by any triangles
				// (by convention, these are always at the end of their section of the array)
				constexpr bool bIncludeCollisionSampleVerts = false;
				if (!bIncludeCollisionSampleVerts)
				{
					int32 MaxUsedIdx = Start;
					int32 FStart = Collection.FaceStart[GeometryIdx];
					int32 FCount = Collection.FaceCount[GeometryIdx];
					for (int32 FIdx = FStart; FIdx < FStart + FCount; FIdx++)
					{
						MaxUsedIdx = FMath::Max(MaxUsedIdx, Collection.Indices[FIdx].GetMax());
					}
					Count = MaxUsedIdx - Start + 1;
				}
				float NormalOffset = CustomVoronoiSettings->NormalOffset;
				if (NormalOffset > 0 || bUseOrderedSkip)
				{
					TArray<FVector3f> Normals;
					Normals.SetNumZeroed(Count);
					TMap<FVector, int32> VertexHash;

					// merge exact duplicate points and average their normals
					for (int32 VIdx = Start; VIdx < Start + Count; VIdx++)
					{
						const FVector& Position = (FVector)Collection.Vertex[VIdx];
						int32& Idx = VertexHash.FindOrAdd(Position);
						if (Idx - 1 == INDEX_NONE)
						{
							Idx = VIdx + 1 - Start;
						}
						Normals[Idx - 1] += Collection.Normal[VIdx];
					}
					for (FVector3f& Normal : Normals)
					{
						Normal.Normalize();
					}
					for (TPair<FVector, int32>& Point : VertexHash)
					{
						const FVector OffsetPosition = (FVector)Point.Key + (FVector)Normals[Point.Value - 1] * NormalOffset;
						const FVector LocalPos = BoneTransform.TransformPosition(OffsetPosition);
						Sites.Add(Context.GetTransform().TransformPosition(LocalPos));
					}
					if (bUseOrderedSkip)
					{
						TArray<float> WeightsOnSourceVertices;
						WeightsOnSourceVertices.SetNumZeroed(Count);
						for (int32 VIdx = Start; VIdx < Start + Count; VIdx++)
						{
							const FVector& Position = (FVector)Collection.Vertex[VIdx];
							int32 NIdx = VertexHash[Position] - 1;
							float& Wt = WeightsOnSourceVertices[NIdx];
							Wt = FMath::Max(Wt, 1 - Normals[NIdx].Dot(Collection.Normal[VIdx]));
						}
						// Now re-order the weights to be 1:1 with the Sites (i.e., order by VertexHash iterator)
						ImportanceWeights.Reserve(ImportanceWeights.Num() + VertexHash.Num());
						for (TPair<FVector, int32>& Point : VertexHash)
						{
							ImportanceWeights.Add(WeightsOnSourceVertices[Point.Value - 1]);
						}
					}
				}
				else
				{
					// if no normal offset, allow Voronoi diagram to drop the duplicate points automatically
					for (int32 VIdx = Start; VIdx < Start + Count; VIdx++)
					{
						const FVector& Position = (FVector)Collection.Vertex[VIdx];
						const FVector LocalPos = BoneTransform.TransformPosition(Position);
						Sites.Add(Context.GetTransform().TransformPosition(LocalPos));
					}
				}
			}
		}
	}
	else if (CustomVoronoiSettings->VoronoiPattern == EVoronoiPattern::Grid)
	{
		auto ToFrac = [](int32 Val, int32 NumVals) -> FVector::FReal
		{
			return (FVector::FReal(Val) + FVector::FReal(.5)) / FVector::FReal(NumVals);
		};
		for (int32 X = 0; X < CustomVoronoiSettings->GridX; ++X)
		{
			FVector::FReal XFrac = ToFrac(X, CustomVoronoiSettings->GridX);
			for (int32 Y = 0; Y < CustomVoronoiSettings->GridY; ++Y)
			{
				FVector::FReal YFrac = ToFrac(Y, CustomVoronoiSettings->GridY);
				for (int32 Z = 0; Z < CustomVoronoiSettings->GridZ; ++Z)
				{
					FVector::FReal ZFrac = ToFrac(Z, CustomVoronoiSettings->GridZ);
					Sites.Emplace(Bounds.Min + FVector(XFrac, YFrac, ZFrac) * Extent);
				}
			}
		}
	}

	if (CustomVoronoiSettings->Variability > 0)
	{
		for (FVector& Site : Sites)
		{
			Site += (RandStream.VRand() * RandStream.FRand() * CustomVoronoiSettings->Variability);
		}
	}

	// randomly remove points based on the skip fraction
	int32 TargetNumSites = Sites.Num() - int32(Sites.Num() * CustomVoronoiSettings->SkipFraction);
	if (CustomVoronoiSettings->SkipFraction > 0)
	{
		if (bUseOrderedSkip)
		{
			UE::Geometry::FPriorityOrderPoints Ordering;
			if (CustomVoronoiSettings->SkipMode == EDownsamplingMode::UniformSpacing)
			{
				Ordering.ComputeUniformSpaced(Sites, ImportanceWeights, TargetNumSites);
			}
			else // CustomVoronoiSettings->SkipMode == EDownsamplingMode::KeepSharpEDownsamplingMode::KeepSharp
			{
				Ordering.ComputeDescendingImportance(ImportanceWeights);
			}

			// empty ordering can happen if we're in a mode w/ no ImportanceWeights but mode is KeepSharp; just leave default order in that case
			if (!Ordering.Order.IsEmpty())
			{
				TArray<FVector> OrderedSites;
				OrderedSites.SetNumUninitialized(TargetNumSites);
				for (int32 Idx = 0; Idx < TargetNumSites; Idx++)
				{
					OrderedSites[Idx] = Sites[Ordering.Order[Idx]];
				}
				Sites = MoveTemp(OrderedSites);
			}
			else
			{
				Sites.SetNum(TargetNumSites);
			}
		}
		else
		{
			while (TargetNumSites < Sites.Num())
			{
				int32 ToRemoveIdx = RandStream.RandHelper(Sites.Num());
				Sites.RemoveAtSwap(ToRemoveIdx, EAllowShrinking::No);
			}
		}
	}

	// Convert newly generated points from world space to local (unscaled) gizmo space
	FTransform ReferenceFrame = GetGizmoTransform();
	ReferenceFrame.RemoveScaling();
	for (FVector& Site : Sites)
	{
		Site = ReferenceFrame.InverseTransformPosition(Site);
	}
	
}

FTransform UFractureToolCustomVoronoi::GetGizmoTransform() const
{
	if (GizmoSettings->IsGizmoEnabled())
	{
		return GizmoSettings->GetTransform();
	}
	else
	{
		return FTransform::Identity;
	}
}

void UFractureToolCustomVoronoi::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	CombinedWorldBounds = FBox(EForceInit::ForceInit);
	for (const FFractureToolContext& Context : FractureContexts)
	{
		CombinedWorldBounds += Context.GetWorldBounds();
	}

	if (CombinedWorldBounds.IsValid && LiveSites.Num() == 0 && FractureContexts.Num() > 0)
	{
		GenerateLivePattern(FractureContexts, FractureContexts[0].GetSeed());
	}

	UpdateVisualizations(FractureContexts);
}

void UFractureToolCustomVoronoi::GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites)
{
	Sites.Append(FrozenSites);
	FTransform Transform = FTransform::Identity;
	if (GizmoSettings->IsGizmoEnabled())
	{
		Transform = GizmoSettings->GetTransform();
	}

	for (FVector Site : LiveSites)
	{
		Sites.Add(Transform.TransformPosition(Site));
	}
}

void UFractureToolCustomVoronoi::SelectedBonesChanged()
{
	GizmoSettings->ResetGizmo();
	Super::SelectedBonesChanged();
}


#undef LOCTEXT_NAMESPACE

