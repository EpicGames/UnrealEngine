// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "FractureToolCutter.h"

#include "FractureToolCustomVoronoi.generated.h"

class FFractureToolContext;

UENUM()
enum class EVoronoiPattern
{
	// Add a single site centered on the gizmo (or multiple if Variability is > 0)
	Centered,
	// Add uniform-random sites
	Uniform,
	// Add a regular grid of points
	Grid,
	// Add a point at every mesh vertex
	MeshVertices,
	// Add a point per vertex of the selected bones
	SelectedBones
};

UENUM()
enum class EDownsamplingMode
{
	// Downsample points by randomly removing points, without considering spacing
	Random,
	// Downsample points so they're spread evenly across space, favoring points on sharp features
	UniformSpacing,
	// Downsample points so points on sharp features are the last to be removed
	KeepSharp
};

UCLASS(config = EditorPerProjectUserSettings)
class UFractureCustomVoronoiSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureCustomVoronoiSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, Variability(0.0f)
	{}

	/** Method to generate next group of voronoi sites */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Pattern"))
	EVoronoiPattern VoronoiPattern = EVoronoiPattern::Uniform;
	
	/** Offset point samples in the vertex normal directions */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (EditCondition = "VoronoiPattern == EVoronoiPattern::MeshVertices || VoronoiPattern == EVoronoiPattern::SelectedBones"))
	float NormalOffset = 0.0f;

	/** Amount to randomly displace each Voronoi site (in cm) */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Variability", UIMin = "0.0", ClampMin = "0.0"))
	float Variability = 0.0f;

	/** Number of Voronoi sites to add */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Sites To Add", ClampMin = "1", UIMax = "1000", ClampMax = "5000",
		EditCondition = "VoronoiPattern != EVoronoiPattern::MeshVertices && VoronoiPattern != EVoronoiPattern::Grid", EditConditionHides))
	int32 SitesToAdd = 20;

	/** Number of sites to add to grid in X */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Sites in X", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "VoronoiPattern == EVoronoiPattern::Grid", EditConditionHides))
	int32 GridX = 5;

	/** Number of sites to add to grid in Y */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Sites in Y", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "VoronoiPattern == EVoronoiPattern::Grid", EditConditionHides))
	int32 GridY = 5;

	/** Number of sites to add to grid in Z */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Sites in Z", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "VoronoiPattern == EVoronoiPattern::Grid", EditConditionHides))
	int32 GridZ = 5;

	/** Fraction of points to skip */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (ClampMin = "0", ClampMax = "1"))
	float SkipFraction = 0.0;

	/** Strategy used for skipping points; only used if SkipFraction is greater than 0 */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites)
	EDownsamplingMode SkipMode;

	//~ TODO: Have reference mesh fall back to the input fracture mesh if not provided here
	/** Static mesh actor to be used as a reference when adding Voronoi sites */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (DisplayName = "Reference Mesh", EditCondition = "VoronoiPattern == EVoronoiPattern::MeshVertices", EditConditionHides))
	TSoftObjectPtr<AStaticMeshActor> ReferenceMesh;

	/** Whether to use the reference mesh actor's transform when placing the Voronoi sites, or center them at the current gizmo location instead */
	UPROPERTY(EditAnywhere, Category = LiveVoronoiSites, meta = (EditCondition = "VoronoiPattern == EVoronoiPattern::MeshVertices", EditConditionHides))
	bool bStartAtActor = false;

	/** Freeze the current live Voronoi Sites based on the current parameters */
	UFUNCTION(CallInEditor, Category = CustomVoronoi)
	void FreezeLiveSites();

	/** Remove all frozen sites */
	UFUNCTION(CallInEditor, Category = CustomVoronoi)
	void ClearFrozenSites();

	/** Unfreeze all frozen sites */
	UFUNCTION(CallInEditor, Category = CustomVoronoi)
	void UnfreezeSites();

	/** Re-run the live Voronoi sites generator, using the current settings and selection bounds */
	UFUNCTION(CallInEditor, Category = CustomVoronoi)
	void RegenerateLiveSites();

	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
};


UCLASS(DisplayName="Custom Voronoi", Category="FractureTools")
class UFractureToolCustomVoronoi : public UFractureToolVoronoiCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolCustomVoronoi(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void SelectedBonesChanged() override;

	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext ) override;

	// CustomVoronoi Voronoi Fracture Input Settings
	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureCustomVoronoiSettings> CustomVoronoiSettings;

	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureTransformGizmoSettings> GizmoSettings;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Shutdown() override;

	// Clear "live" voronoi sites (sites that are being edited by current settings / gizmo) -- they will repopulate on next call to GenerateVoronoiSites()
	void ClearLiveSites()
	{
		LiveSites.Empty();
	}

	void FreezeLiveSites()
	{
		FTransform Transform = GetGizmoTransform();
		for (const FVector& Site : LiveSites)
		{
			FrozenSites.Add(Transform.TransformPosition(Site));
		}
		LiveSites.Empty();
	}

	void ClearFrozenSites()
	{
		FrozenSites.Empty();
	}

	void UnfreezeSites()
	{
		FTransform Transform = GetGizmoTransform();
		for (const FVector& Site : FrozenSites)
		{
			LiveSites.Add(Transform.InverseTransformPosition(Site));
		}
		FrozenSites.Empty();
	}

	// Re-calculate the "live transformed" voronoi sites based on current settings
	void GenerateLivePattern(const TArray<FFractureToolContext>& FractureContexts, int32 RandomSeed);

	void FractureContextChanged() override;

protected:
	void GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites) override;

	FTransform GetGizmoTransform() const;

	// Sites that are not currently live-edited
	TArray<FVector> FrozenSites;

	// Voronoi sites that are currently being regenerated and moved
	TArray<FVector> LiveSites;

	// Bounds of the combined, currently-selected fracture contexts, to be used the next time the fracture pattern is generated
	FBox CombinedWorldBounds = FBox(EForceInit::ForceInit);
};