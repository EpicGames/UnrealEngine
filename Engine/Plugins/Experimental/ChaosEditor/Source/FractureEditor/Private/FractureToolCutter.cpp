// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolCutter.h"

#include "FractureTool.h" // for LogFractureTool
#include "FractureToolContext.h"
#include "InteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "FractureEditorMode.h"
#include "ToolSetupUtil.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Components/DynamicMeshComponent.h"

#include "FractureEngineMaterials.h"

#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "DynamicMesh/MeshTransforms.h"

#include "FractureToolBackgroundTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolCutter)

using namespace UE::Fracture;

#define LOCTEXT_NAMESPACE "FractureToolCutter"


UFractureTransformGizmoSettings::UFractureTransformGizmoSettings(const FObjectInitializer& ObjInit) : Super(ObjInit)
{

}

void UFractureTransformGizmoSettings::ResetGizmo(bool bResetRotation)
{
	if (!TransformGizmo || !TransformProxy || !TransformGizmo->GetGizmoActor())
	{
		// doesn't have a valid gizmo to reset
		return;
	}
	if (!bUseGizmo || !AttachedCutter)
	{
		TransformGizmo->SetVisibility(false);
		return;
	}
	FBox CombinedBounds = AttachedCutter->GetCombinedBounds(AttachedCutter->GetFractureToolContexts());
	TransformGizmo->SetVisibility((bool)CombinedBounds.IsValid);
	if (CombinedBounds.IsValid)
	{
		if (bCenterOnSelection && !GIsTransacting)
		{
			FTransform Transform = TransformProxy->GetTransform();
			Transform.SetTranslation(CombinedBounds.GetCenter());
			if (bResetRotation)
			{
				Transform.SetRotation(FQuat::Identity);
			}
			TransformGizmo->SetNewGizmoTransform(Transform);
		}
	}
}

void UFractureTransformGizmoSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFractureTransformGizmoSettings, bUseGizmo))
	{
		if (AttachedCutter)
		{
			AttachedCutter->UpdateUseGizmo(bUseGizmo);
			ResetGizmo();
		}
	}
}

void UFractureTransformGizmoSettings::TransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (bUseGizmo && AttachedCutter)
	{
		AttachedCutter->FractureContextChanged();
	}
}

void UFractureTransformGizmoSettings::Setup(UFractureToolCutterBase* Cutter, ETransformGizmoSubElements GizmoElements)
{
	AttachedCutter = Cutter;
	UFractureEditorMode* Mode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	UsedToolsContext = Mode->GetInteractiveToolsContext();
	if (ensure(UsedToolsContext && AttachedCutter))
	{
		UInteractiveGizmoManager* GizmoManager = UsedToolsContext->GizmoManager;
		TransformProxy = NewObject<UTransformProxy>(this);
		TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, GizmoElements, this);
		// TODO: Stop setting bUseEditorCompositing on gizmo components like this, once gizmo rendering is improved
		// This is a hack to make gizmos more visible when the translucent fracture bone selection material is on screen
		// This hack unfortunately makes the gizmos dithered when they're behind objects, but at least they're relatively visible
		for (UActorComponent* Component : TransformGizmo->GetGizmoActor()->GetComponents())
		{
			UGizmoBaseComponent* GizmoComponent = Cast<UGizmoBaseComponent>(Component);
			if (GizmoComponent)
			{
				GizmoComponent->bUseEditorCompositing = true;
			}
		}
		TransformGizmo->SetActiveTarget(TransformProxy);
		TransformProxy->OnTransformChanged.AddUObject(this, &UFractureTransformGizmoSettings::TransformChanged);
		AttachedCutter->UpdateUseGizmo(bUseGizmo);
		ResetGizmo();
	}
}

void UFractureTransformGizmoSettings::Shutdown()
{
	if (UsedToolsContext)
	{
		UsedToolsContext->GizmoManager->DestroyAllGizmosByOwner(this);
		UsedToolsContext = nullptr;
	}
	TransformGizmo = nullptr;
	TransformProxy = nullptr;
}

void UFractureCutterSettings::TransferNoiseSettings(FNoiseSettings& NoiseSettingsOut)
{
	NoiseSettingsOut.Amplitude = Amplitude;
	NoiseSettingsOut.Frequency = Frequency;
	NoiseSettingsOut.Lacunarity = Lacunarity;
	NoiseSettingsOut.Persistence = Persistence;
	NoiseSettingsOut.Octaves = OctaveNumber;
	NoiseSettingsOut.PointSpacing = PointSpacing;
}

UFractureToolCutterBase::UFractureToolCutterBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	CutterSettings = GetMutableDefault<UFractureCutterSettings>();
	CollisionSettings = GetMutableDefault<UFractureCollisionSettings>();
}

bool UFractureToolCutterBase::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}
	
	return true;
}

TArray<FFractureToolContext> UFractureToolCutterBase::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component, or for each individual bone if Group Fracture is not used.
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection();
		if (IsValid(RestCollection))
		{

			// Generate a context for each selected node
			FFractureToolContext FullSelection(GeometryCollectionComponent);
			int32 FullSelectionSeed = CutterSettings->RandomSeed > -1 ? CutterSettings->RandomSeed : DefaultRandomSeed;
			FullSelection.SetSeed(FullSelectionSeed);
			FullSelection.ConvertSelectionToRigidNodes();
			FullSelection.RandomReduceSelection(CutterSettings->ChanceToFracture);

			// Update global transforms and bounds		
			const TManagedArray<FTransform3f>& Transform = FullSelection.GetGeometryCollection()->GetAttribute<FTransform3f>("Transform", FGeometryCollection::TransformGroup);
			const TManagedArray<int32>& TransformToGeometryIndex = FullSelection.GetGeometryCollection()->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
			const TManagedArray<FBox>& BoundingBoxes = FullSelection.GetGeometryCollection()->GetAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);

			TArray<FTransform> Transforms;
			GeometryCollectionAlgo::GlobalMatrices(Transform, FullSelection.GetGeometryCollection()->Parent, Transforms);

			TMap<int32, FBox> BoundsToBone;
			int32 TransformCount = Transform.Num();
			for (int32 Index = 0; Index < TransformCount; ++Index)
			{
				if (TransformToGeometryIndex[Index] > INDEX_NONE)
				{
					BoundsToBone.Add(Index, BoundingBoxes[TransformToGeometryIndex[Index]].TransformBy(Transforms[Index]));
				}
			}

			if (CutterSettings->bGroupFracture)
			{
				FBox Bounds(ForceInit);
				for (int32 BoneIndex : FullSelection.GetSelection())
				{
					if (TransformToGeometryIndex[BoneIndex] > INDEX_NONE)
					{
						Bounds += BoundsToBone[BoneIndex];
					}
				}
				FullSelection.SetBounds(Bounds);

				Contexts.Add(FullSelection);
			}
			else
			{
				// Generate a context for each selected node
				for (int32 Index : FullSelection.GetSelection())
				{
					FFractureToolContext& FractureContext = Contexts.Emplace_GetRef(GeometryCollectionComponent);

					TArray<int32> Selection;
					Selection.Add(Index);
					FractureContext.SetSelection(Selection);
					FractureContext.SetSeed(FullSelectionSeed + Index);
					FractureContext.SetBounds(BoundsToBone[Index]);
				}
			}
		}
	}

	return Contexts;
}


FBox UFractureToolCutterBase::GetCombinedBounds(const TArray<FFractureToolContext>& Contexts) const
{
	FBox CombinedBounds(EForceInit::ForceInit);
	for (const FFractureToolContext& FractureContext : Contexts)
	{
		CombinedBounds += FractureContext.GetWorldBounds();
	}
	return CombinedBounds;
}


void UFractureToolCutterBase::UpdateDefaultRandomSeed()
{
	DefaultRandomSeed = FMath::Rand();
}


void UFractureToolCutterBase::PostFractureProcess(const FFractureToolContext& FractureContext, int32 FirstNewGeometryIndex)
{
	Super::PostFractureProcess(FractureContext, FirstNewGeometryIndex);

	// Apply Internal Material ID to new fracture geometry.
	int32 InternalMaterialID = CutterSettings->GetInternalMaterialID();
	if (InternalMaterialID > INDEX_NONE)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		FFractureEngineMaterials::SetMaterialOnGeometryAfter(Collection, FirstNewGeometryIndex, FFractureEngineMaterials::ETargetFaces::InternalFaces, InternalMaterialID);
		Collection.ReindexMaterials();
	}
}

void FCellNoisePreviewOp::CalculateResult(FProgressCancel* Progress)
{
	Result->SetNum(ComputedDiagrams.Num());
	for (int32 DiagramIdx = 0; DiagramIdx < ComputedDiagrams.Num(); ++DiagramIdx)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		FCellDiagramResult& Info = ComputedDiagrams[DiagramIdx];
		int32 NumCells = Info.Diagram.NumCells;
		int32 KeepCellStep = 1.0f / FMath::Clamp(KeepCellsFrac, FLT_EPSILON, 1.0f);
		auto KeepCellFunc = [KeepCellStep](int32 Idx) -> bool
		{
			return 0 == (Idx % KeepCellStep);
		};
		Info.Diagram.DiscardCells(KeepCellFunc, true);
		FCellNoisePreviewResult& Preview = (*Result)[DiagramIdx];

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		CreateCuttingSurfacePreview(Info.Diagram, Info.Bounds, Grout, Info.NoiseSeed, Preview.NoiseMesh, KeepCellFunc, Info.Transform, Progress, Info.Transform.GetTranslation());

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		FVector Origin = Info.Transform.GetTranslation();
		const UE::Geometry::FTransformSRT3d GeoTransform(Info.Transform);
		MeshTransforms::ApplyTransform(Preview.NoiseMesh, GeoTransform, true);

		Preview.BoneIdx = Info.BoneIdx;
		Preview.Transform = Info.Transform;
		Preview.SourceComponentIdx = Info.SourceComponentIdx;
	}
}

void FVoronoiCellsOp::CalculateResult(FProgressCancel* Progress)
{
	for (FVoronoiDiagramInput& DiagramInfo : Diagrams)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}
		FVector Origin = DiagramInfo.Transform.GetTranslation();
		for (FVector& Site : DiagramInfo.Sites)
		{
			Site -= Origin;
		}
		DiagramInfo.Bounds.Max -= Origin;
		DiagramInfo.Bounds.Min -= Origin;

		FVoronoiDiagram Voronoi(DiagramInfo.Sites, DiagramInfo.Bounds, .1f);
		Result->Emplace(FPlanarCells(DiagramInfo.Sites, Voronoi), DiagramInfo.SourceComponentIdx, DiagramInfo.BoneIdx, DiagramInfo.Transform, DiagramInfo.Bounds, DiagramInfo.NoiseSeed);
	}
}

UFractureToolVoronoiCutterBase::UFractureToolVoronoiCutterBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	for (int32 ii = 0; ii < 100; ++ii)
	{
		// This is adapted from FColor::MakeRandomColor() but desaturated a bit
		const uint8 Hue = (uint8)(FMath::FRand() * 255.f);
		FColor DesatRandomColor = FLinearColor::MakeFromHSV8(Hue, 190, 255).ToFColor(true);
		Colors.Add(DesatRandomColor);
	}

	// Hotfix: make sure to cancel background task if the tool is still open when closing the editor
	FCoreDelegates::OnEnginePreExit.AddWeakLambda(this, [this]()
		{
			UE::Fracture::CancelBackgroundTask<FVoronoiCellsOp>(ComputeCells);
			UE::Fracture::CancelBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview);
		});
}

void UFractureToolVoronoiCutterBase::OnTick(float DeltaTime)
{
	ComputeRelaunchDelay -= DeltaTime;
	if (bDiagramUpdated)
	{
		if (ComputeCells)
		{
			CancelBackgroundTask<FVoronoiCellsOp>(ComputeCells);
			CancelBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview);
			ComputeRelaunchDelay = .5;
		}
		if (ComputeRelaunchDelay <= 0)
		{
			ComputeCells = StartBackgroundTask<FVoronoiCellsOp>(MakeUnique<FVoronoiCellsOp>(MoveTemp(DiagramInputs)));
			bDiagramUpdated = false;
		}
	}
	if (!CutterSettings->bDrawNoisePreview)
	{
		ClearMeshes();
		CancelBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview);
	}
	if (ComputeCells)
	{
		if (ComputeNoisePreview)
		{
			CancelBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview);
		}
		TickBackgroundTask<FVoronoiCellsOp>(ComputeCells, false, [&](TUniquePtr<TArray<FCellDiagramResult>>&& Result) -> bool
		{
			AddLineVisualizations(*Result.Get());
			ComputeRelaunchDelay = 0;
			// TODO: Consider saving this result and passing it to the actual fracture Execute, rather than recomputing it (maybe not worth added complexity though)
			TUniquePtr<FCellNoisePreviewOp> NoiseOp = MakeUnique<FCellNoisePreviewOp>(MoveTemp(*Result));

			// Transfer noise settings
			FNoiseSettings Settings;
			CutterSettings->TransferNoiseSettings(Settings);
			for (FCellDiagramResult& DiagramInfo : NoiseOp->ComputedDiagrams)
			{
				DiagramInfo.Diagram.SetNoise(Settings);
			}
			NoiseOp->PointSpacing = CollisionSettings->GetPointSpacing();
			NoiseOp->Grout = CutterSettings->Grout;
			NoiseOp->KeepCellsFrac = CutterSettings->FractionPreviewCells;
			ensure(!ComputeNoisePreview);
			if (CutterSettings->bDrawNoisePreview)
			{
				ComputeNoisePreview = StartBackgroundTask<FCellNoisePreviewOp>(MoveTemp(NoiseOp));
			}
			return true;
		});
	}
	if (ComputeNoisePreview)
	{
		TickBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview, false, [&](TUniquePtr<TArray<FCellNoisePreviewResult>>&& Result) -> bool
		{
			AddNoiseVisualizations(*Result.Get());
			return true;
		});
	}
}

void UFractureToolVoronoiCutterBase::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (CutterSettings->bDrawSites)
	{
		EnumerateVisualizationMapping(SitesMappings, VoronoiSites.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			PDI->DrawPoint(VoronoiSites[Idx] + ExplodedVector, FLinearColor(.62f, .94f, .84f), 4.f, SDPG_Foreground);
		});
	}

	UpdateLineSetExplodedVectors();
}

void UFractureToolVoronoiCutterBase::UpdateLineSetExplodedVectors()
{
	bool bHasMeshes = VoronoiNoisePreviews.Num() == VoronoiLineSets.Num();
	EnumerateVisualizationMapping(EdgesMappings, VoronoiLineSets.Num(), [&](int32 Idx, FVector ExplodedVector)
	{
		if (!VoronoiLineSets[Idx])
		{
			return;
		}
		// TODO: If we add diagrams even when visibility is toggled off, we also need to update visibility here, e.g.: VoronoiLineSets[Idx]->SetVisibility(CutterSettings->bDrawDiagram);
		AActor* Actor = VoronoiLineSets[Idx]->GetOwner();
		VoronoiLineSets[Idx]->SetRelativeLocation(Actor->GetTransform().InverseTransformVector(ExplodedVector));
		if (bHasMeshes)
		{
			AActor* MeshActor = VoronoiNoisePreviews[Idx]->GetOwner();
			VoronoiNoisePreviews[Idx]->SetRelativeLocation(MeshActor->GetTransform().InverseTransformVector(ExplodedVector));
		}
	});
}

void UFractureToolVoronoiCutterBase::AddNoiseVisualizations(TArray<FCellNoisePreviewResult>& NoiseResults)
{
	ClearMeshes();

	int32 NumCollections = NoiseResults.Num();
	if (!ensure(VisualizedCollections.Num() == NumCollections))
	{
		return;
	}

	if (!GEditor)
	{
		return;		
	}
	UWorld* World = GEditor->GetEditorWorldContext().World();

	for (int32 CollectionIdx = 0; CollectionIdx < NumCollections; ++CollectionIdx)
	{
		UGeometryCollectionComponent* Component = VisualizedCollections[CollectionIdx].Get();
		if (!ensure(Component))
		{
			continue;
		}
		FCellNoisePreviewResult& Res = NoiseResults[CollectionIdx];

		UDynamicMeshComponent* DynamicMeshComponent = NewObject<UDynamicMeshComponent>(Component);
		DynamicMeshComponent->SetSceneProxyVerifyUsedMaterials(false);
		DynamicMeshComponent->RegisterComponent();
		VoronoiNoisePreviews.Add(DynamicMeshComponent);
		DynamicMeshComponent->GetMesh()->Copy(Res.NoiseMesh);
		DynamicMeshComponent->NotifyMeshUpdated();
		DynamicMeshComponent->SetIsEditable(false);

		// TODO: this material-access logic can go to the base class and be used by FractureToolPlaneCut as well
		// Note the ToolSetupUtil functions optionally take a UInteractiveToolManager to provide fallback materials,
		// but fracture mode does not set that up, so they may return a null material.
		UMaterialInterface* NoiseMaterial = ToolSetupUtil::GetDefaultSculptMaterial(nullptr);
		if (!NoiseMaterial)
		{
			NoiseMaterial = ToolSetupUtil::GetDefaultMaterial();
		}
		DynamicMeshComponent->SetNumMaterials(1);
		DynamicMeshComponent->SetMaterial(0, NoiseMaterial);
	}
}

void UFractureToolVoronoiCutterBase::AddLineVisualizations(TArray<FCellDiagramResult>& DiagramResults)
{
	ClearEdges();

	int32 NumCollections = DiagramResults.Num();
	if (!ensure(VisualizedCollections.Num() == NumCollections))
	{
		RestoreEditorViewFlags();
		return;
	}

	int32 TotalSites = 0;
	for (const FCellDiagramResult& Result : DiagramResults)
	{
		TotalSites += Result.Diagram.NumCells;
	}
	// even though the diagrams are computed in the background, creating the line set component w/ all the voronoi diagrams edges causes a large hitch
	// if the number of sites becomes too large, and the becomes diagram unreadably dense, so it's better to not draw it
	constexpr int32 MaxSitesToShowEdges = 100000;
	if (TotalSites > MaxSitesToShowEdges)
	{
		UE_LOGF(LogFractureTool, Warning, "Voronoi diagram(s) number of sites too large; will not display Voronoi diagram edges");
		RestoreEditorViewFlags();
		return;
	}
	for (int32 CollectionIdx = 0; CollectionIdx < DiagramResults.Num(); ++CollectionIdx)
	{
		const FCellDiagramResult& Result = DiagramResults[CollectionIdx];
		check(CollectionIdx == Result.SourceComponentIdx);
		EdgesMappings.AddMapping(CollectionIdx, Result.BoneIdx, VoronoiLineSets.Num());
		UGeometryCollectionComponent* Component = VisualizedCollections[CollectionIdx].Get();
		if (!ensure(Component))
		{
			continue;
		}
		int32 NumBoundaries = Result.Diagram.PlaneBoundaries.Num();
		if (CutterSettings->bDrawDiagram && NumBoundaries > 0)
		{
			ULineSetComponent* VoronoiLineSet = NewObject<ULineSetComponent>(Component);
			VoronoiLineSet->SetupAttachment(Component);
			VoronoiLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(nullptr, false));
			// TODO: If we switch to adding diagrams even when visibility is toggled off, we also need to update visibility here, 
			// e.g.: VoronoiLineSet->SetVisibility(CutterSettings->bDrawDiagram);
			VoronoiLineSet->RegisterComponent();
			int32 NumLines = 0;
			for (int32 BoundaryIdx = 0; BoundaryIdx < NumBoundaries; ++BoundaryIdx)
			{
				NumLines += Result.Diagram.PlaneBoundaries[BoundaryIdx].Num();
			}
			int32 LinesPerIndexHint = NumLines / NumBoundaries + 1;
			FVector Origin = Result.Transform.GetTranslation();
			VoronoiLineSet->AddLines(
				NumBoundaries,
				[&](int32 BoundaryIdx, TArray<FRenderableLine>& LinesOut)
				{
					const TArray<int32>& Boundary = Result.Diagram.PlaneBoundaries[BoundaryIdx];
					TPair<int32, int32> Cells = Result.Diagram.PlaneCells[BoundaryIdx];
					int32 Cell = Cells.Key > -1 ? Cells.Key : Cells.Value;
					for (int32 PrevIdx = Boundary.Num() - 1, Idx = 0; Idx < Boundary.Num(); PrevIdx = Idx++)
					{
						FVector A = Result.Transform.InverseTransformPosition(Result.Diagram.PlaneBoundaryVertices[Boundary[PrevIdx]] + Origin);
						FVector B = Result.Transform.InverseTransformPosition(Result.Diagram.PlaneBoundaryVertices[Boundary[Idx]] + Origin);

						LinesOut.Emplace(A, B, Colors[Cell % 100], 1.3, .1);
					}
				},
				LinesPerIndexHint,
				false);
			VoronoiLineSets.Add(VoronoiLineSet);
		}
	}

	if (VoronoiLineSets.Num() > 0)
	{
		OverrideEditorViewFlagsForLineRendering();
		UpdateLineSetExplodedVectors();
	}
	else
	{
		RestoreEditorViewFlags();
	}
}

void UFractureToolVoronoiCutterBase::UpdateVisualizations(TArray<FFractureToolContext>& FractureContexts)
{
	ClearVisualizations();

	int32 MaxSitesToShowSites = 1000000; // PDI struggles to render the site positions above this
	bool bEstAboveMaxSites = false;

	DiagramInputs.Reset(FractureContexts.Num());
	bDiagramUpdated = true;

	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		if (!FractureContext.GetBounds().IsValid) // skip contexts w/ invalid bounds
		{
			continue;
		}
		int32 CollectionIdx = VisualizedCollections.Emplace(FractureContext.GetGeometryCollectionComponent());
		int32 BoneIdx = FractureContext.GetSelection().Num() == 1 ? FractureContext.GetSelection()[0] : INDEX_NONE;
		SitesMappings.AddMapping(CollectionIdx, BoneIdx, VoronoiSites.Num());

		// Store the voronoi diagrams we'd like to preview; Tick() will launch the background thread to compute them
		TArray<FVector> LocalVoronoiSites;
		GenerateVoronoiSites(FractureContext, LocalVoronoiSites);

		FVoronoiDiagramInput& Diagram = DiagramInputs.Emplace_GetRef();
		Diagram.Bounds = GetVoronoiBounds(FractureContext, LocalVoronoiSites);
		Diagram.Sites = LocalVoronoiSites;
		Diagram.SourceComponentIdx = CollectionIdx;
		Diagram.Transform = FractureContext.GetTransform();
		Diagram.BoneIdx = BoneIdx;
		Diagram.NoiseSeed = FractureContext.GetSeed();

		// Voronoi site point visualization doesn't need background compute, but is skipped if there are too many sites
		if (LocalVoronoiSites.Num() * FractureContexts.Num() > MaxSitesToShowSites || VoronoiSites.Num() + LocalVoronoiSites.Num() > MaxSitesToShowSites)
		{
			UE_LOGF(LogFractureTool, Warning, "Voronoi diagram(s) number of sites too large; will not display Voronoi diagram sites");
			ClearVisualizations();
			break;
		}
		VoronoiSites.Append(LocalVoronoiSites);
	}

	RestoreEditorViewFlags();
}

void UFractureToolVoronoiCutterBase::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();
	UpdateVisualizations(FractureContexts);
}


class FVoronoiFractureOp : public FGeometryCollectionFractureOperator
{
public:
	FVoronoiFractureOp(const FGeometryCollection& SourceCollection) : FGeometryCollectionFractureOperator(SourceCollection)
	{}

	virtual ~FVoronoiFractureOp() = default;

	TArray<int> Selection;
	FBox Bounds;
	TArray<FVector> Sites;
	TOptional<FNoiseSettings> NoiseSettings;
	float PointSpacing;
	float Grout;
	int Seed;
	FTransform Transform;
	bool bSplitIslands = true;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		// Assume Voronoi diagram only takes ~1% of the total time to compute
		FProgressCancel::FProgressScope VoronoiDiagramProgress =
			FProgressCancel::CreateScopeTo(Progress, .01f, LOCTEXT("ComputingVoronoiDiagramMessage", "Computing Voronoi Diagram"));

		FVector Origin = Transform.GetTranslation();
		for (FVector& Site : Sites)
		{
			Site -= Origin;
		}
		Bounds.Min -= Origin;
		Bounds.Max -= Origin;
		FVoronoiDiagram Voronoi(Sites, Bounds, .1f);

		FPlanarCells VoronoiPlanarCells = FPlanarCells(Sites, Voronoi);
		VoronoiPlanarCells.InternalSurfaceMaterials.NoiseSettings = NoiseSettings;

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		VoronoiDiagramProgress.Done();

		// All remaining work is assigned to the mesh fracture
		FProgressCancel::FProgressScope FractureMeshProgress =
			FProgressCancel::CreateScopeTo(Progress, 1, LOCTEXT("FractureMeshMessage", "Fracturing Mesh"));

		ResultGeometryIndex = CutMultipleWithPlanarCells(VoronoiPlanarCells, *CollectionCopy, Selection, Grout, PointSpacing, Seed, Transform, true, true, Progress, Origin, FIslandSplitSettings(bSplitIslands, CloseVertexDistance, VertexToSurfaceBridgeDistance));
		
		SetResult(MoveTemp(CollectionCopy));
	}
};


int32 UFractureToolVoronoiCutterBase::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		TArray<FVector> Sites;
		GenerateVoronoiSites(FractureContext, Sites);
		FBox VoronoiBounds = GetVoronoiBounds(FractureContext, Sites);

		TUniquePtr<FVoronoiFractureOp> VoronoiOp = MakeUnique<FVoronoiFractureOp>(*(FractureContext.GetGeometryCollection()));
		VoronoiOp->Bounds = VoronoiBounds;
		VoronoiOp->Selection = FractureContext.GetSelection();
		VoronoiOp->Grout = CutterSettings->Grout;
		VoronoiOp->PointSpacing = CollisionSettings->GetPointSpacing();
		VoronoiOp->Sites = Sites;
		VoronoiOp->bSplitIslands = CutterSettings->bSplitIslands;
		VoronoiOp->CloseVertexDistance = CutterSettings->CloseVertexDistance;
		VoronoiOp->VertexToSurfaceBridgeDistance = CutterSettings->VertexToSurfaceBridgeDistance;
		if (CutterSettings->Amplitude > 0.0f)
		{
			FNoiseSettings Settings;
			CutterSettings->TransferNoiseSettings(Settings);
			VoronoiOp->NoiseSettings = Settings;
		}
		VoronoiOp->Seed = FractureContext.GetSeed();
		VoronoiOp->Transform = FractureContext.GetTransform();

		int Result = RunCancellableGeometryCollectionOp<FVoronoiFractureOp>(*(FractureContext.GetGeometryCollection()),
			MoveTemp(VoronoiOp), LOCTEXT("ComputingVoronoiFractureMessage", "Computing Voronoi Fracture"));
		return Result;
	}

	return INDEX_NONE;
}

void UFractureToolVoronoiCutterBase::ClearMeshes()
{
	for (UDynamicMeshComponent* NoisePreview : VoronoiNoisePreviews)
	{
		if (NoisePreview)
		{
			NoisePreview->UnregisterComponent();
			NoisePreview->DestroyComponent();
		}
	}
	VoronoiNoisePreviews.Empty();
}

FBox UFractureToolVoronoiCutterBase::GetVoronoiBounds(const FFractureToolContext& FractureContext, const TArray<FVector>& Sites) const
{
	FBox VoronoiBounds = FractureContext.GetWorldBounds(); 
	if (Sites.Num() > 0)
	{
		VoronoiBounds += FBox(Sites);
	}
	
	return VoronoiBounds.ExpandBy(CutterSettings->GetMaxVertexMovement() + KINDA_SMALL_NUMBER);
}

#undef LOCTEXT_NAMESPACE

