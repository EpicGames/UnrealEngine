// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformProxy.h"
#include "Drawing/LineSetComponent.h"

// for background compute of preview geometry
#include "PlanarCut.h"
#include "FractureToolBackgroundTask.h"

#include "FractureToolCutter.generated.h"

class FFractureToolContext;
class UDynamicMeshComponent;
struct FNoiseSettings;

/** Settings specifically related to the one-time destructive fracturing of a mesh **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureCutterSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureCutterSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, RandomSeed(-1)
		, ChanceToFracture(1.0)
		, bGroupFracture(true)
		, bDrawSites(false)
		, bDrawDiagram(true)
		, Amplitude(0.0f)
		, Frequency(0.1f)
		, Persistence(0.5f)
		, Lacunarity(2.0f)
		, OctaveNumber(4)
		, PointSpacing(10)
	{}

	void TransferNoiseSettings(FNoiseSettings& NoiseSettingsOut);

	/** Material to set for internal faces on fracture. 'Automatic' will use the most common 'internal' material in each geometry, or the last valid material if no internal faces are found */
	UPROPERTY(EditAnywhere, Category = Materials, meta = (TransientToolProperty, DisplayName = "Internal Material", GetOptions = GetMaterialNamesFunc, NoResetToDefault))
	FString InternalMaterial;

	UFUNCTION()
	const TArray<FString>& GetMaterialNamesFunc() { return ActiveMaterialNamesList; }

	void UpdateActiveMaterialNames(TArray<FString> InMaterialNamesList)
	{
		ActiveMaterialNamesList = InMaterialNamesList;
		if (!ActiveMaterialNamesList.Contains(InternalMaterial))
		{
			InternalMaterial = ActiveMaterialNamesList.IsEmpty() ? FString() : ActiveMaterialNamesList[0];
		}
	}

	// Get the user-chosen internal material ID, or INDEX_NONE if 'Automatic' is chosen
	int32 GetInternalMaterialID()
	{
		int32 NameIndex = ActiveMaterialNamesList.Find(InternalMaterial);
		return FMath::Max((int32)INDEX_NONE, NameIndex - 1);
	}

	/** Random number generator seed for repeatability. If the value is -1, a different random seed will be used every time, otherwise the specified seed will always be used */
	UPROPERTY(EditAnywhere, Category = CommonFracture, meta = (DisplayName = "Random Seed", UIMin = "-1", UIMax = "1000", ClampMin = "-1"))
	int32 RandomSeed;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = CommonFracture, meta = (DisplayName = "Chance To Fracture Per Bone", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float ChanceToFracture;

	/** Generate a fracture pattern across all selected meshes.  */
	UPROPERTY(EditAnywhere, Category = CommonFracture, meta = (EditCondition = "bGroupFractureToggleEnabled", HideEditConditionToggle, DisplayName = "Group Fracture"))
	bool bGroupFracture;

	// This flag allows tools to disable the above bGroupFracture option if/when it is not applicable
	UPROPERTY()
	bool bGroupFractureToggleEnabled = true;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "CommonFracture|Split Islands")
	bool bSplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "CommonFracture|Split Islands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = "0.0"))
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "CommonFracture|Split Islands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = "0.0"))
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = CommonFracture, meta = (UIMin = "0.0", ClampMin = "0.0", EditCondition = "bGroutSettingEnabled", HideEditConditionToggle, EditConditionHides))
	float Grout = 0.0f;

	// This flag allows tools to disable the above Grout setting if/when it's not applicable
	UPROPERTY()
	bool bGroutSettingEnabled = true;

	/** Draw points marking the centers of pieces to be cut out by the fracture pattern.  */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (EditCondition = "bDrawSitesToggleEnabled", HideEditConditionToggle, EditConditionHides, DisplayName = "Draw Sites"))
	bool bDrawSites;

	// This flag allows tools to disable the above bDrawSites option if/when it is not applicable
	UPROPERTY()
	bool bDrawSitesToggleEnabled = true;

	/** Draw the edges of the fracture pattern.  */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Draw Diagram"))
	bool bDrawDiagram;

	/** Whether to show a solid preview of the cutting geometry, including any noise displacement */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (EditCondition = "bNoisePreviewToggleEnabled", HideEditConditionToggle, EditConditionHides))
	bool bDrawNoisePreview = true;

	// This flag allows tools to disable the above bDrawNoisePreview option if/when it is not applicable
	UPROPERTY()
	bool bNoisePreviewToggleEnabled = true;

	/** Fraction of cells to show in noise preview */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bDrawNoisePreview && bDrawSitesToggleEnabled", HideEditConditionToggle, EditConditionHides))
	float FractionPreviewCells = .1f;

	/** Scale of the noise preview plane */
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = ".01", ClampMax = "1000", UIMin = ".25", UIMax = "10", EditCondition = "bDrawNoisePreview && bNoisePreviewHasScale", HideEditConditionToggle, EditConditionHides))
	double NoisePreviewScale = 1.;

	// This flag allows tools to disable the noise preview scale setting if it's not applicable
	UPROPERTY()
	bool bNoisePreviewHasScale = false;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0.0", EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	float Amplitude;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	float Frequency;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0.01", UIMax = "1.0", EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	float Persistence;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "1.0", UIMax = "4.0", EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	float Lacunarity;

	/** 
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (ClampMin = "1", UIMax = "8", EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	int32 OctaveNumber;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = Noise, meta = (DisplayName = "Point Spacing", UIMin = "1", ClampMin = "0.1", EditCondition = "bNoiseSettingsEnabled", HideEditConditionToggle, EditConditionHides))
	float PointSpacing;

	// This flag allows tools to disable the above Noise settings if/when they are not applicable
	UPROPERTY()
	bool bNoiseSettingsEnabled = true;

	/// Get the maximum distance a vertex could be moved by a combination of grout and noise
	float GetMaxVertexMovement()
	{
		float MaxDisp = Grout;
		float AmplitudeScaled = Amplitude;
		for (int32 OctaveIdx = 0; OctaveIdx < OctaveNumber; OctaveIdx++, AmplitudeScaled *= Persistence)
		{
			MaxDisp += FMath::Abs(AmplitudeScaled);
		}
		return MaxDisp;
	}

private:

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> ActiveMaterialNamesList;
};

/** Settings related to the collision properties of the fractured mesh pieces */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureCollisionSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureCollisionSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit) {}

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 * 
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = Collision)
	bool bAddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (UIMin = "1", ClampMin = "0.1", EditCondition = "bAddSamplesForCollision"))
	float PointSpacing = 50.0f;

	// Get point spacing or the special value of 0 if adding collision samples is disabled
	float GetPointSpacing()
	{
		return bAddSamplesForCollision ? PointSpacing : 0.0f;
	}

	// TODO: add remeshing options here as well
};

UCLASS(Abstract, DisplayName = "Cutter Base", Category = "FractureTools")
class UFractureToolCutterBase : public UFractureInteractiveTool
{
public:
	GENERATED_BODY()

	UFractureToolCutterBase(const FObjectInitializer& ObjInit);

	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("Fracture", "ExecuteFracture", "Fracture")); }

	virtual bool CanExecute() const override;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	virtual void FractureContextChanged() override
	{
		UpdateDefaultRandomSeed();
	}

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override
	{
		Super::Setup(InToolkit);
		CutterSettings->UpdateActiveMaterialNames(GetSelectedComponentMaterialNames(true));
		CutterSettings->OwnerTool = this;
		CollisionSettings->OwnerTool = this;
		ConfigureCutterSettings();
	}

	// Set tool-specific defaults for the cutter settings (e.g., disable grout/noise if needed)
	virtual void ConfigureCutterSettings()
	{
		CutterSettings->bGroupFractureToggleEnabled = true;
		CutterSettings->bDrawSitesToggleEnabled = true;
		CutterSettings->bNoisePreviewToggleEnabled = true;
		CutterSettings->bNoisePreviewHasScale = false;
		CutterSettings->bGroutSettingEnabled = true;
		CutterSettings->bNoiseSettingsEnabled = true;
	}

	virtual void SelectedBonesChanged() override
	{
		Super::SelectedBonesChanged();
		CutterSettings->UpdateActiveMaterialNames(GetSelectedComponentMaterialNames(true));
	}

	FBox GetCombinedBounds(const TArray<FFractureToolContext>& Contexts) const;

	// Post-processing; sets user-specified internal material on fracture (if not automatic)
	virtual void PostFractureProcess(const FFractureToolContext& FractureContext, int32 FirstNewGeometryIndex) override;

	virtual void UpdateUseGizmo(bool bUseGizmo)
	{
		SetMandateGroupFracture(bUseGizmo);
	}

	void SetMandateGroupFracture(bool bMandateGroupFracture)
	{
		CutterSettings->bGroupFractureToggleEnabled = !bMandateGroupFracture;
		if (bMandateGroupFracture)
		{
			CutterSettings->bGroupFracture = true;
		}
	}

protected:
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureCutterSettings> CutterSettings;

	UPROPERTY(EditAnywhere, Category = Collision)
	TObjectPtr<UFractureCollisionSettings> CollisionSettings;

	/// Manage which seed to use when no specific random seed is specified, to control when the seed changes
	/// (primarily to avoid the seed changing between creating the preview and doing the actual cut)
	void UpdateDefaultRandomSeed();
	int32 DefaultRandomSeed = 0;

};


struct FVoronoiDiagramInput
{
	FTransform Transform;
	int32 SourceComponentIdx;
	int32 BoneIdx;
	TArray<FVector> Sites;
	FBox Bounds;
	int32 NoiseSeed;
};

struct FCellDiagramResult
{
	FCellDiagramResult(FPlanarCells Diagram, int32 SourceComponentIdx, int32 BoneIdx, FTransform Transform, FBox Bounds, int32 NoiseSeed)
		: Diagram(Diagram), SourceComponentIdx(SourceComponentIdx), BoneIdx(BoneIdx), Transform(Transform), Bounds(Bounds), NoiseSeed(NoiseSeed)
	{}

	FPlanarCells Diagram;
	int32 SourceComponentIdx;
	int32 BoneIdx;
	FTransform Transform;
	FBox Bounds;
	int32 NoiseSeed;
};

struct FCellNoisePreviewResult
{
	UE::Geometry::FDynamicMesh3 NoiseMesh;
	int32 SourceComponentIdx;
	int32 BoneIdx;
	FTransform Transform;
};

class FCellNoisePreviewOp : public UE::Geometry::TGenericDataOperator<TArray<FCellNoisePreviewResult>>
{
public:
	FCellNoisePreviewOp(TArray<FCellDiagramResult>&& ComputedDiagrams) : ComputedDiagrams(MoveTemp(ComputedDiagrams))
	{}
	virtual ~FCellNoisePreviewOp() = default;

	TArray<FCellDiagramResult> ComputedDiagrams;
	float PointSpacing;
	float Grout;

	float KeepCellsFrac = .1f;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override;
};

class FVoronoiCellsOp : public UE::Geometry::TGenericDataOperator<TArray<FCellDiagramResult>>
{
public:

	FVoronoiCellsOp(TArray<FVoronoiDiagramInput>&& Inputs) : Diagrams(MoveTemp(Inputs))
	{}

	virtual ~FVoronoiCellsOp() = default;

	TArray<FVoronoiDiagramInput> Diagrams;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override;
};


UCLASS(Abstract, DisplayName = "Voronoi Base", Category = "FractureTools")
class UFractureToolVoronoiCutterBase : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolVoronoiCutterBase(const FObjectInitializer& ObjInit);

	virtual void OnTick(float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;

protected:
	virtual void GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites) {}

	FBox GetVoronoiBounds(const FFractureToolContext& FractureContext, const TArray<FVector>& Sites) const;

	virtual void ClearVisualizations()
	{
		Super::ClearVisualizations();
		VoronoiSites.Empty();
		SitesMappings.Empty();
		ClearEdges();
		ClearMeshes();
		if (ComputeCells)
		{
			UE::Fracture::CancelBackgroundTask<FVoronoiCellsOp>(ComputeCells);
			UE::Fracture::CancelBackgroundTask<FCellNoisePreviewOp>(ComputeNoisePreview);
			ComputeRelaunchDelay = .5;
		}
	}

	void ClearEdges()
	{
		CellMember.Empty();
		EdgesMappings.Empty();
		for (ULineSetComponent* Lines : VoronoiLineSets)
		{
			if (Lines)
			{
				Lines->DestroyComponent();
			}
		}
		VoronoiLineSets.Empty();
	}

	void ClearMeshes();

	virtual void UpdateVisualizations(TArray<FFractureToolContext>& FractureContexts);
	virtual void AddLineVisualizations(TArray<FCellDiagramResult>& DiagramResults);
	virtual void AddNoiseVisualizations(TArray<FCellNoisePreviewResult>& NoiseResults);

private:
	UPROPERTY()
	TArray<TObjectPtr<ULineSetComponent>> VoronoiLineSets;
	UPROPERTY()
	TArray<TObjectPtr<UDynamicMeshComponent>> VoronoiNoisePreviews;


	//
	// Background compute
	//
	TUniquePtr<UE::Fracture::TBackgroundOpExecuter<FVoronoiCellsOp>> ComputeCells = nullptr;
	TUniquePtr<UE::Fracture::TBackgroundOpExecuter<FCellNoisePreviewOp>> ComputeNoisePreview = nullptr;

	TArray<FVoronoiDiagramInput> DiagramInputs;
	bool bDiagramUpdated = false;
	float ComputeRelaunchDelay = 0;

	TArray<int32> CellMember;
	TArray<FVector> VoronoiSites;

	TArray<FColor> Colors;

	FVisualizationMappings SitesMappings, EdgesMappings;

	void UpdateLineSetExplodedVectors();
};


/// This helps create a 3D transform gizmo that can be used to adjust fracture placement
// Note it is tailored to UFractureToolCutterBase, and expects Setup(), Shutdown()
// and ResetGizmo() to be called on tool setup, shutdown, and selection change respectively
UCLASS(config = EditorPerProjectUserSettings)
class UFractureTransformGizmoSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureTransformGizmoSettings(const FObjectInitializer & ObjInit);

	void Setup(UFractureToolCutterBase* Cutter, ETransformGizmoSubElements GizmoElements = ETransformGizmoSubElements::StandardTranslateRotate);
	void Shutdown();

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	void TransformChanged(UTransformProxy* Proxy, FTransform Transform);

	void ResetGizmo(bool bResetRotation = false);

	bool IsGizmoEnabled()
	{
		return bUseGizmo && TransformGizmo && TransformGizmo->IsVisible();
	}

	FTransform GetTransform()
	{
		return TransformProxy->GetTransform();
	}

	/** Use a 3D rigid transform gizmo to place the fracture pattern.  Only supports grouped fracture. */
	UPROPERTY(EditAnywhere, Category = PlacementControls, meta = (EditCondition = "bShowUseGizmoOption", EditConditionHides, HideEditConditionToggle))
	bool bUseGizmo = true;

	/** Recenter the gizmo to the center of the selection when selection changes */
	UPROPERTY(EditAnywhere, Category = PlacementControls, meta = (EditCondition = "bUseGizmo", EditConditionHides))
	bool bCenterOnSelection = true;

	UPROPERTY()
	bool bShowUseGizmoOption = true;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

protected:

	UPROPERTY()
	TObjectPtr<UFractureToolCutterBase> AttachedCutter = nullptr;

	// the tools context responsible for the gizmo
	UPROPERTY()
	TObjectPtr<UInteractiveToolsContext> UsedToolsContext = nullptr;
};

