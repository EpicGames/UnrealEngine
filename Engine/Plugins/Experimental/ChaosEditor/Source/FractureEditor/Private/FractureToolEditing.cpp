// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolEditing.h"

#include "Editor.h"
#include "Dialogs/Dialogs.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "FractureToolContext.h"
#include "ScopedTransaction.h"

#include "PlanarCut.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolEditing)

#define LOCTEXT_NAMESPACE "FractureToolEditing"


FText UFractureToolDeleteBranch::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolDeleteBranch", "Delete"));
}

FText UFractureToolDeleteBranch::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolDeleteBranchTooltip", "Delete all nodes in selected branch. Empty clusters will be eliminated."));
}

FSlateIcon UFractureToolDeleteBranch::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.DeleteBranch");
}

void UFractureToolDeleteBranch::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "DeleteBranch", "Prune", "Delete all nodes in selected branch. Empty clusters will be eliminated.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->DeleteBranch = UICommandInfo;
}

void UFractureToolDeleteBranch::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteBranch", "Prune"));

		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);
				FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get();
				UGeometryCollection* FracturedGeometryCollection = Context.GetFracturedGeometryCollection();

				const TManagedArray<int32>& ExemplarIndex = GeometryCollection->ExemplarIndex;
				const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;

				// Removing the root node amounts to full deletion -- we don't allow this here.
				Context.RemoveRootNodes();
				Context.Sanitize();

				TArray<int32> NodesForDeletion;

				for (int32 Select : Context.GetSelection())
				{
					FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(Children, Select, NodesForDeletion);
				}

				// Clean up any embedded geometry removal
				TArray<int32> UninstancedExemplars;
				UninstancedExemplars.Reserve(NodesForDeletion.Num());
				for (int32 DeleteNode : NodesForDeletion)
				{
					if (ExemplarIndex[DeleteNode] > INDEX_NONE)
					{
						if ((--FracturedGeometryCollection->EmbeddedGeometryExemplar[ExemplarIndex[DeleteNode]].InstanceCount) < 1)
						{
							UE_LOGF(LogFractureTool, Warning, "Exemplar Index %d is empty. Removing Exemplar from Geometry Collection.", ExemplarIndex[DeleteNode]);
							UninstancedExemplars.Add(ExemplarIndex[DeleteNode]);
						}
					}
				}

				UninstancedExemplars.Sort();
				FracturedGeometryCollection->RemoveExemplars(UninstancedExemplars);
				GeometryCollection->ReindexExemplarIndices(UninstancedExemplars);

				NodesForDeletion.Sort();
				GeometryCollection->RemoveElements(FGeometryCollection::TransformGroup, NodesForDeletion);

				FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeometryCollection);

				Context.GetGeometryCollectionComponent()->InitializeEmbeddedGeometry();

				// Proximity is invalidated.
				ClearProximity(Context.GetGeometryCollection().Get());

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolMergeSelected::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolMergeSelected", "Merge"));
}

FText UFractureToolMergeSelected::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolMergeSelectedTooltip", "Merge all selected nodes into one node."));
}

FSlateIcon UFractureToolMergeSelected::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.MergeSelected");
}

void UFractureToolMergeSelected::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "MergeSelected", "GeoMrg", "Merge all selected nodes into one node.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->MergeSelected = UICommandInfo;
}

void UFractureToolMergeSelected::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("MergeSelected", "Merge Selected"));

		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);
				FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get();

				Context.Sanitize();

				const TArray<int32>& NodesForMerge = Context.GetSelection();

				constexpr bool bBooleanUnion = false;
				MergeAllSelectedBones(*GeometryCollection, NodesForMerge, bBooleanUnion);

				// Proximity is invalidated.
				ClearProximity(Context.GetGeometryCollection().Get());

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}


FText UFractureToolSplitSelected::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolSplitSelected", "Split"));
}

FText UFractureToolSplitSelected::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolSplitSelectedTooltip", "Split all selected nodes into their connected component parts."));
}

FSlateIcon UFractureToolSplitSelected::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SplitSelected");
}

void UFractureToolSplitSelected::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SplitSelected", "Split", "Split all selected nodes into their connected component parts.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SplitSelected = UICommandInfo;
}

void UFractureToolSplitSelected::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("SplitSelected", "Split Selected"));

		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);
				FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get();

				Context.Sanitize();

				TArray<int32> NodesForSplit;

				for (int32 Select : Context.GetSelection())
				{
					FGeometryCollectionClusteringUtility::GetLeafBones(GeometryCollection, Select, true, NodesForSplit);
				}

				SplitIslands(*GeometryCollection, NodesForSplit, 0, nullptr);

				// Proximity is invalidated.
				ClearProximity(Context.GetGeometryCollection().Get());

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolHide::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolHide", "Hide"));
}

FText UFractureToolHide::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolHideTooltip", "Set all geometry in selected branch to invisible."));
}

FSlateIcon UFractureToolHide::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Hide");
}

void UFractureToolHide::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Hide", "Hide", "Set all geometry in selected branch to invisible.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Hide = UICommandInfo;
}

void UFractureToolHide::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		FScopedTransaction Transaction(LOCTEXT("FractureHideTransaction", "Hide"));
		
		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);
				FGeometryCollection* GeometryCollection = Edit.GetRestCollection()->GetGeometryCollection().Get();
				UGeometryCollection* FracturedGeometryCollection = Context.GetFracturedGeometryCollection();

				Context.ConvertSelectionToRigidNodes();

				const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection->TransformToGeometryIndex;
				const TManagedArray<int32>& FaceStart = GeometryCollection->FaceStart;
				const TManagedArray<int32>& FaceCount = GeometryCollection->FaceCount;
				TManagedArray<bool>& Visible = GeometryCollection->Visible;


				const TArray<int32>& Selection = Context.GetSelection();
				for (int32 Idx : Selection)
				{
					// Iterate the faces in the geometry of this rigid node and set invisible.
					if (TransformToGeometryIndex[Idx] > INDEX_NONE)
					{
						int32 CurrFace = FaceStart[TransformToGeometryIndex[Idx]];
						for (int32 FaceOffset = 0; FaceOffset < FaceCount[TransformToGeometryIndex[Idx]]; ++FaceOffset)
						{
							Visible[CurrFace + FaceOffset] = false;
						}
					}
				}

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}



FText UFractureToolUnhide::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolUnhide", "Hide"));
}

FText UFractureToolUnhide::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolUnhideTooltip", "Set all geometry in selected branch to visible."));
}

FSlateIcon UFractureToolUnhide::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Unhide");
}

void UFractureToolUnhide::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Unhide", "Unhide", "Set all geometry in selected branch to visible.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Unhide = UICommandInfo;
}

void UFractureToolUnhide::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		FScopedTransaction Transaction(LOCTEXT("FractureUnhideTransaction", "Unhide"));
		
		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic);
				FGeometryCollection* GeometryCollection = Edit.GetRestCollection()->GetGeometryCollection().Get();
				UGeometryCollection* FracturedGeometryCollection = Context.GetFracturedGeometryCollection();

				Context.ConvertSelectionToRigidNodes();

				const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection->TransformToGeometryIndex;
				const TManagedArray<int32>& FaceStart = GeometryCollection->FaceStart;
				const TManagedArray<int32>& FaceCount = GeometryCollection->FaceCount;
				TManagedArray<bool>& Visible = GeometryCollection->Visible;


				const TArray<int32>& Selection = Context.GetSelection();
				for (int32 Idx : Selection)
				{
					// Iterate the faces in the geometry of this rigid node and set invisible.
					if (TransformToGeometryIndex[Idx] > INDEX_NONE)
					{
						int32 CurrFace = FaceStart[TransformToGeometryIndex[Idx]];
						for (int32 FaceOffset = 0; FaceOffset < FaceCount[TransformToGeometryIndex[Idx]]; ++FaceOffset)
						{
							Visible[CurrFace + FaceOffset] = true;
						}
					}
				}

				Refresh(Context, Toolkit, true);
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}

UFractureToolValidate::UFractureToolValidate(const FObjectInitializer& ObjInit) : Super(ObjInit)
{
	ValidationSettings = NewObject<UFractureValidateSettings>(GetTransientPackage(), UFractureValidateSettings::StaticClass());
	ValidationSettings->OwnerTool = this;
}


FText UFractureToolValidate::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolValidate", "Validate"));
}

FText UFractureToolValidate::GetTooltipText() const
{
	return FText(NSLOCTEXT("FractureToolEditingOps", "FractureToolValidateTooltip", "Ensure that geometry collection is valid and clean."));
}

FSlateIcon UFractureToolValidate::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Validate");
}

void UFractureToolValidate::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Validate", "Validate", "Ensure that geometry collection is valid and clean.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->Validate = UICommandInfo;
}

TArray<UObject*> UFractureToolValidate::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ValidationSettings);
	return Settings;
}

void UFractureToolValidate::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		bool bUpdatedCollections = false;

		TSet<UGeometryCollectionComponent*> GeomCompSelection;
		GetSelectedGeometryCollectionComponents(GeomCompSelection);
		for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
		{
			bool bDirty = false;

			FGeometryCollectionEdit GeometryCollectionEdit = GeometryCollectionComponent->EditRestCollection();
			if (UGeometryCollection* GeometryCollectionObject = GeometryCollectionEdit.GetRestCollection())
			{
				TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
				if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
				{
					TManagedArray<int32>& TransformToGeometry = GeometryCollection->TransformToGeometryIndex;
					constexpr bool bClustersCanHaveGeometry = true;
					if (!bClustersCanHaveGeometry)
					{
						// Optionally ensure that clusters do not point to geometry (currently disabled; keeping the geometry can be useful)
						const int32 ElementCount = TransformToGeometry.Num();
						for (int32 Idx = 0; Idx < ElementCount; ++Idx)
						{
							if (GeometryCollection->IsClustered(Idx) && TransformToGeometry[Idx] != INDEX_NONE)
							{
								TransformToGeometry[Idx] = INDEX_NONE;
								UE_LOGF(LogFractureTool, Warning, "Removed geometry index from cluster %d.", Idx);
								bDirty = true;
							}
						}
					}

					// Remove any unreferenced geometry
					if (ValidationSettings->bRemoveUnreferencedGeometry)
					{
						TManagedArray<int32>& TransformIndex = GeometryCollection->TransformIndex;
						const int32 GeometryCount = TransformIndex.Num();

						TArray<int32> RemoveGeometry;
						RemoveGeometry.Reserve(GeometryCount);

						for (int32 Idx = 0; Idx < GeometryCount; ++Idx)
						{
							if ((TransformIndex[Idx] == INDEX_NONE) || (TransformToGeometry[TransformIndex[Idx]] != Idx))
							{
								RemoveGeometry.Add(Idx);
								UE_LOGF(LogFractureTool, Warning, "Removed dangling geometry at index %d.", Idx);
								bDirty = true;
							}
						}

						if (RemoveGeometry.Num() > 0)
						{
							FManagedArrayCollection::FProcessingParameters Params;
							Params.bDoValidation = false; // for perf reasons
							GeometryCollection->RemoveElements(FGeometryCollection::GeometryGroup, RemoveGeometry);
						}
					}

					if (ValidationSettings->bRemoveClustersOfOne)
					{
						if (FGeometryCollectionClusteringUtility::RemoveClustersOfOnlyOneChild(GeometryCollection))
						{
							UE_LOGF(LogFractureTool, Warning, "Removed one or more clusters of only one child.");
							bDirty = true;
						}
					}

					if (ValidationSettings->bRemoveDanglingClusters)
					{
						if (FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeometryCollection))
						{
							UE_LOGF(LogFractureTool, Warning, "Removed one or more dangling clusters.");
							bDirty = true;
						}
					}

					if (bDirty)
					{
						FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
						AddSingleRootNodeIfRequired(GeometryCollectionObject);

						// Update resource data to correctly reflect modified geometry collection data
						GeometryCollectionObject->RebuildRenderData();
					}
				}
			}

			GeometryCollectionComponent->InitializeEmbeddedGeometry();

			if (bDirty)
			{
				// reset bone selection because bones may have been deleted
				FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();
				EditBoneColor.ResetBoneSelection();
				EditBoneColor.ResetHighlightedBones();

				// flag that at least one geometry collection has changed
				bUpdatedCollections = true;
			}
		}

		if (bUpdatedCollections)
		{
			Toolkit->RegenerateHistogram();
			Toolkit->RegenerateOutliner();
		}

		Toolkit->OnSetLevelViewValue(-1);
		Toolkit->SetOutlinerComponents(GeomCompSelection.Array());	
	}
}

#undef LOCTEXT_NAMESPACE


