// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolEditing.generated.h"

UCLASS(DisplayName = "Delete Branch", Category = "FractureTools")
class UFractureToolDeleteBranch : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolDeleteBranch(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};

UCLASS(DisplayName = "Merge Selected", Category = "FractureTools")
class UFractureToolMergeSelected : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolMergeSelected(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};

UCLASS(DisplayName = "Split Selected", Category = "FractureTools")
class UFractureToolSplitSelected : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolSplitSelected(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};



UCLASS(DisplayName = "Hide", Category = "FractureTools")
class UFractureToolHide : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolHide(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Unhide", Category = "FractureTools")
class UFractureToolUnhide : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolUnhide(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


/** Settings for the Validate tool **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureValidateSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureValidateSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
	}

	/** Find and remove any unused geometry data */
	UPROPERTY(EditAnywhere, Category = CleanUnused)
	bool bRemoveUnreferencedGeometry = true;

	/** Whether to collapse any clusters with only a single child */
	UPROPERTY(EditAnywhere, Category = Clustering)
	bool bRemoveClustersOfOne = false;

	/** Remove dangling clusters -- Note this can invalidate caches */
	UPROPERTY(EditAnywhere, Category = Clustering)
	bool bRemoveDanglingClusters = false;

};




UCLASS(DisplayName = "Validate", Category = "FractureTools")
class UFractureToolValidate : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolValidate(const FObjectInitializer& ObjInit);

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("ValidateTool", "ExecuteValidate", "Validate")); }

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override
	{
		checkNoEntry(); // Updates handled via the Execute() call instead (TODO: is that best?)
		return -1;
	}

	UPROPERTY(EditAnywhere, Category = Validation)
	TObjectPtr<UFractureValidateSettings> ValidationSettings;
};
