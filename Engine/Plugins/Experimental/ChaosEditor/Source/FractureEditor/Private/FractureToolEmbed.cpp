// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolEmbed.h"

#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "AssetSelection.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConversion.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Chaos/Convex.h"
#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolEmbed)


#define LOCTEXT_NAMESPACE "FractureToolEmbed"

FText UFractureToolAddEmbeddedGeometry::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolAddEmbeddedGeometry", "Embed"));
}

FText UFractureToolAddEmbeddedGeometry::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolAddEmbeddedGeometryTooltip", "Embed selected static mesh as passive geometry parented to selected bone. Will be lost if GeometryCollection is Reset!"));
}

FSlateIcon UFractureToolAddEmbeddedGeometry::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.AddEmbeddedGeometry");
}

void UFractureToolAddEmbeddedGeometry::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "AddEmbeddedGeometry", "Embed", "Embed selected static mesh as passive geometry parented to selected bone. Will be lost if GeometryCollection is Reset!", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->AddEmbeddedGeometry = UICommandInfo;
}

bool UFractureToolAddEmbeddedGeometry::CanExecute() const
{
	return (IsStaticMeshSelected() && IsGeometryCollectionSelected());
}

void UFractureToolAddEmbeddedGeometry::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		// Get selected static meshes
		TArray<UStaticMeshComponent*> SelectedStaticMeshComponents = GetSelectedStaticMeshComponents();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				FGeometryCollectionEdit GeometryCollectionEdit = Context.GetGeometryCollectionComponent()->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysicsDynamic, true /*bShapeIsUnchanged*/);
				UGeometryCollection* FracturedGeometryCollection = GeometryCollectionEdit.GetRestCollection();

				FFractureToolContext::FGeometryCollectionPtr GeometryCollection = Context.GetGeometryCollection();
				const TManagedArray<FTransform3f>& Transform = GeometryCollection->Transform;
				const TManagedArray<int32>& Parent = GeometryCollection->Parent;

				int32 StartTransformCount = Transform.Num();

				const FTransform TargetActorTransform(Context.GetGeometryCollectionComponent()->GetOwner()->GetTransform());

				Context.ConvertSelectionToRigidNodes();
				const TArray<int32>& SelectedBones = Context.GetSelection();
				for (const int32 SelectedBone : SelectedBones)
				{
					FTransform BoneGlobalTransform = GeometryCollectionAlgo::GlobalMatrix(Transform, Parent, SelectedBone);

					for (UStaticMeshComponent* SelectedStaticMeshComponent : SelectedStaticMeshComponents)
					{
						const AActor* Actor = SelectedStaticMeshComponent->GetOwner();
						check(Actor)

							const FTransform SMActorTransform(Actor->GetTransform());

						UStaticMesh* ComponentStaticMesh = SelectedStaticMeshComponent->GetStaticMesh();

						FTransform ComponentTransform = SMActorTransform.GetRelativeTransform(TargetActorTransform);
						FTransform BoneTransform = ComponentTransform.GetRelativeTransform(BoneGlobalTransform);

						int32 ExemplarIndex = FracturedGeometryCollection->AttachEmbeddedGeometryExemplar(ComponentStaticMesh);
						if (GeometryCollection->AppendEmbeddedInstance(ExemplarIndex, SelectedBone, BoneTransform))
						{
							FracturedGeometryCollection->EmbeddedGeometryExemplar[ExemplarIndex].InstanceCount++;
						}
					}
				}

				Context.GenerateGuids(StartTransformCount);

				Context.GetGeometryCollectionComponent()->InitializeEmbeddedGeometry();
				Refresh(Context, Toolkit);

				FracturedGeometryCollection->MarkPackageDirty();
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}

TArray<UStaticMeshComponent*> UFractureToolAddEmbeddedGeometry::GetSelectedStaticMeshComponents()
{
	TArray<UStaticMeshComponent*> SelectedStaticMeshComponents;

	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			// We don't want to include any static meshes owned by a geometry collection.
			// (This might be an ISMC or swap out static geometry.)
			if (Actor->FindComponentByClass<UGeometryCollectionComponent>())
			{
				continue;
			}

			TArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents, true);
			SelectedStaticMeshComponents.Append(StaticMeshComponents);
		}
	}

	return SelectedStaticMeshComponents;
}



FText UFractureToolAutoEmbedGeometry::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolAutoEmbedGeometry", "Auto"));
}

FText UFractureToolAutoEmbedGeometry::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolAutoEmbedGeometryTooltip", "Embed selected static meshes as passive geometry parented to nearest bone. Will be lost if GeometryCollection is Reset!"));
}

FSlateIcon UFractureToolAutoEmbedGeometry::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.AutoEmbedGeometry");
}

void UFractureToolAutoEmbedGeometry::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "AutoEmbedGeometry", "Auto", "Embed selected static meshes as passive geometry parented to nearest bone. Will be lost if GeometryCollection is Reset!", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->AutoEmbedGeometry = UICommandInfo;
}

bool UFractureToolAutoEmbedGeometry::CanExecute() const
{
	return (IsStaticMeshSelected() && IsGeometryCollectionSelected());
}

void UFractureToolAutoEmbedGeometry::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		// Get selected static meshes
		TArray<UStaticMeshComponent*> SelectedStaticMeshComponents = GetSelectedStaticMeshComponents();
		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		// For each static mesh component, we iterate all the convex hulls and determine which convex hull best contains the worldspace pivot of the static mesh.
		for (UStaticMeshComponent* SelectedStaticMeshComponent : SelectedStaticMeshComponents)
		{
			// Static Mesh world space location
			FVector SMLocation = SelectedStaticMeshComponent->GetComponentLocation();

			// Determine the "closest" bone
			FFractureToolContext::FGeometryCollectionPtr ClosestGeometryCollection;
			int32 ClosestConvex = INDEX_NONE;
			Chaos::FReal ClosestPhi = TNumericLimits<float>::Max();
			UGeometryCollectionComponent* ClosestComponent = nullptr;
			FFractureToolContext* ClosestContext = nullptr;

			for (FFractureToolContext& Context : Contexts)
			{
				if (Context.GetGeometryCollection())
				{
					FGeometryCollectionEdit Edit(Context.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic, true /*bShapeIsUnchanged*/);

					FGeometryCollection* GeometryCollection = Context.GetGeometryCollection().Get();
					FGeometryCollectionConvexUtility::FGeometryCollectionConvexData ConvexData = FGeometryCollectionConvexUtility::GetValidConvexHullData(GeometryCollection);
					const TManagedArray<FTransform3f>& Transform = GeometryCollection->Transform;
					const TManagedArray<int32>& Parent = GeometryCollection->Parent;
					const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
					TArray<FTransform> BoneGlobalTransforms;
					GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, BoneGlobalTransforms);

					FTransform WorldToComponent = Context.GetGeometryCollectionComponent()->GetComponentToWorld().Inverse();
					FVector ComponentSpaceLocation = WorldToComponent.TransformPosition(SMLocation);

					int32 NumConvex = ConvexData.ConvexHull.Num();
					for (int32 TransformIndex = 0; TransformIndex < Transform.Num(); TransformIndex++)
					{
						if (SimulationType[TransformIndex] != FGeometryCollection::ESimulationTypes::FST_Rigid)
						{
							continue;
						}

						FVector BoneSpaceLocation = BoneGlobalTransforms[TransformIndex].InverseTransformPosition(ComponentSpaceLocation);
						for (int32 ConvexIndex : ConvexData.TransformToConvexIndices[TransformIndex])
						{
							Chaos::FVec3 Normal;
							Chaos::FReal Phi = ConvexData.ConvexHull[ConvexIndex]->PhiWithNormal(BoneSpaceLocation, Normal);
							UE_LOGF(LogTemp, Warning, " Cone %d Phi %f", TransformIndex, Phi);
							if (Phi < ClosestPhi)
							{
								ClosestPhi = Phi;
								ClosestGeometryCollection = Context.GetGeometryCollection();
								ClosestConvex = ConvexIndex;
								ClosestComponent = Context.GetGeometryCollectionComponent();
								ClosestContext = &Context;
							}
						}
					}
				}
			}

			if (ClosestGeometryCollection.IsValid() && ClosestComponent && ClosestContext)
			{
				// Which bone points to the closest convex?
				const TManagedArray<FTransform3f>& Transform = ClosestGeometryCollection->Transform;
				const TManagedArray<int32>& Parent = ClosestGeometryCollection->Parent;
				const TManagedArray<TSet<int32>>& TransformToConvexIndices =
					ClosestGeometryCollection->GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);

				int32 BoneIndex = INDEX_NONE;
				for (int32 SearchIndex = 0; SearchIndex < TransformToConvexIndices.Num(); SearchIndex++)
				{
					const TSet<int32>& ConvexSet = TransformToConvexIndices[SearchIndex];
					if (ConvexSet.Contains(ClosestConvex))
					{
						BoneIndex = SearchIndex;
						break;
					}
				}

				if (BoneIndex > INDEX_NONE)
				{
					// We found the closest bone, now we embed the geometry.
					FGeometryCollectionEdit GeometryCollectionEdit = ClosestComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysicsDynamic, true /*bShapeIsUnchanged*/);
					UGeometryCollection* FracturedGeometryCollection = GeometryCollectionEdit.GetRestCollection();

					FTransform BoneGlobalTransform = GeometryCollectionAlgo::GlobalMatrix(Transform, Parent, BoneIndex);

					const FTransform TargetActorTransform(ClosestComponent->GetOwner()->GetTransform());
					const AActor* Actor = SelectedStaticMeshComponent->GetOwner();
					check(Actor)

					const FTransform SMActorTransform(Actor->GetTransform());
					FTransform ComponentTransform = SMActorTransform.GetRelativeTransform(TargetActorTransform);
					FTransform BoneTransform = ComponentTransform.GetRelativeTransform(BoneGlobalTransform);

					UStaticMesh* ComponentStaticMesh = SelectedStaticMeshComponent->GetStaticMesh();
					int32 ExemplarIndex = FracturedGeometryCollection->AttachEmbeddedGeometryExemplar(ComponentStaticMesh);
					if (ClosestGeometryCollection->AppendEmbeddedInstance(ExemplarIndex, BoneIndex, BoneTransform))
					{
						FracturedGeometryCollection->EmbeddedGeometryExemplar[ExemplarIndex].InstanceCount++;
					}

					// Get a guid generated for the new instance
					TManagedArray<FGuid>& Guids = ClosestGeometryCollection->ModifyAttribute<FGuid>("GUID", FGeometryCollection::TransformGroup);
					Guids[Guids.Num()-1] = FGuid::NewGuid();

					// #todo there might be a lot of these -- collect and put outside the loop.
					ClosestComponent->InitializeEmbeddedGeometry();
					Refresh(*ClosestContext, Toolkit);
					FracturedGeometryCollection->MarkPackageDirty();
				}
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}

TArray<UStaticMeshComponent*> UFractureToolAutoEmbedGeometry::GetSelectedStaticMeshComponents()
{
	TArray<UStaticMeshComponent*> SelectedStaticMeshComponents;

	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			// We don't want to include any static meshes owned by a geometry collection.
			// (This might be an ISMC or swap out static geometry.)
			if (Actor->FindComponentByClass<UGeometryCollectionComponent>())
			{
				continue;
			}

			TArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents, true);
			SelectedStaticMeshComponents.Append(StaticMeshComponents);
		}
	}

	return SelectedStaticMeshComponents;
}






UFractureToolFlushEmbeddedGeometry::UFractureToolFlushEmbeddedGeometry(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	FlushEmbeddedGeometrySettings = NewObject<UFractureToolFlushEmbeddedGeometrySettings>(GetTransientPackage(), UFractureToolFlushEmbeddedGeometrySettings::StaticClass());
	FlushEmbeddedGeometrySettings->OwnerTool = this;
}


FText UFractureToolFlushEmbeddedGeometry::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolFlushEmbeddedGeometry", "Flush"));
}


FText UFractureToolFlushEmbeddedGeometry::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolCFlushEmbeddedGeometryToolTip", "Strip all instanced embedded geometry from the geometry collection."));
}

FText UFractureToolFlushEmbeddedGeometry::GetApplyText() const
{
	return FText(NSLOCTEXT("Fracture", "ExecuteFlushEmbeddedGeometry", "Flush Embedded Geometry"));
}

FSlateIcon UFractureToolFlushEmbeddedGeometry::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.FlushEmbeddedGeometry");
}


TArray<UObject*> UFractureToolFlushEmbeddedGeometry::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(FlushEmbeddedGeometrySettings);
	return Settings;
}

void UFractureToolFlushEmbeddedGeometry::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "FlushEmbeddedGeometry", "Flush", "Strip all instanced embedded geometry from the geometry collection.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->FlushEmbeddedGeometry = UICommandInfo;
}

void UFractureToolFlushEmbeddedGeometry::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> Contexts = GetFractureToolContexts();

		for (FFractureToolContext& Context : Contexts)
		{
			if (Context.GetGeometryCollection())
			{
				if (!Context.GetGeometryCollection()->HasAttribute("ExemplarIndex", FGeometryCollection::TransformGroup))
				{
					continue;
				}

				const TManagedArray<int32>& ExemplarIndex = Context.GetGeometryCollection()->GetAttribute<int32>("ExemplarIndex", FGeometryCollection::TransformGroup);
				const TManagedArray<FTransform3f>& Transforms = Context.GetGeometryCollection()->GetAttribute<FTransform3f>("Transform", FGeometryCollection::TransformGroup);
				const TManagedArray<int32>& ParentIndices = Context.GetGeometryCollection()->GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);

				TArray<FMatrix> GlobalMatrices;
				GeometryCollectionAlgo::GlobalMatrices(Transforms, ParentIndices, GlobalMatrices);

				TArray<int32> RemovalList;
				RemovalList.Reserve(ExemplarIndex.Num());

				TMap<int32, TArray<FTransform>> ExemplarTransforms;

				// Filter for instanced embedded geometry
				for (int32 TransformIndex = 0; TransformIndex < ExemplarIndex.Num(); ++TransformIndex)
				{
					if (ExemplarIndex[TransformIndex] > INDEX_NONE)
					{
						RemovalList.Add(TransformIndex);
						TArray<FTransform>& InstanceTransforms = ExemplarTransforms.FindOrAdd(ExemplarIndex[TransformIndex]);
						InstanceTransforms.Add(FTransform(GlobalMatrices[TransformIndex]));
					}
				}

				if (RemovalList.Num())
				{
					// Remove all instanced embedded geometry.
					FGeometryCollectionEdit GeometryCollectionEdit = Context.GetGeometryCollectionComponent()->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysicsDynamic, true /*bShapeIsUnchanged*/);
					UGeometryCollection* FracturedCollection = GeometryCollectionEdit.GetRestCollection();
					FracturedCollection->GetGeometryCollection()->RemoveElements(FGeometryCollection::TransformGroup, RemovalList);

					if (FlushEmbeddedGeometrySettings->bExtractAsStaticMeshActors)
					{
						FActorSpawnParameters SpawnParameters;

						FTransform ComponentSpace = Context.GetGeometryCollectionComponent()->GetComponentTransform();

						if (UWorld* TargetWorld = Context.GetGeometryCollectionComponent()->GetOwner()->GetWorld())
						{
							// Place Static Mesh Actors
							for (auto It = ExemplarTransforms.CreateConstIterator(); It; ++It)
							{
								FSoftObjectPath ExemplarPath = FracturedCollection->EmbeddedGeometryExemplar[It->Key].StaticMeshExemplar;
								UObject* LoadedExemplarAsset = ExemplarPath.TryLoad();
								if (UActorFactory* ActorFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject(LoadedExemplarAsset))
								{

									for (const FTransform& Transform : It->Value)
									{
										ActorFactory->CreateActor(LoadedExemplarAsset, TargetWorld->GetCurrentLevel(), Transform * ComponentSpace, SpawnParameters);
									}
								}

							}
						}

					}

					// Remove all Exemplars from asset.
					FracturedCollection->EmbeddedGeometryExemplar.Empty();

					Context.GetGeometryCollectionComponent()->InitializeEmbeddedGeometry();

					FScopedColorEdit ScopedEdit(Context.GetGeometryCollectionComponent());
					ScopedEdit.ResetBoneSelection();
					Refresh(Context, Toolkit);
					FracturedCollection->MarkPackageDirty();
				}
			}
		}

		SetOutlinerComponents(Contexts, Toolkit);
	}
}

#undef LOCTEXT_NAMESPACE
