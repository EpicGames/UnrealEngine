// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolEmbed.generated.h"


UCLASS()
class UFractureToolAddEmbeddedGeometry : public UFractureActionTool
{
public:
	GENERATED_BODY()

		UFractureToolAddEmbeddedGeometry(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual bool CanExecute() const override;

private:
	static TArray<UStaticMeshComponent*>  GetSelectedStaticMeshComponents();
};


UCLASS()
class UFractureToolAutoEmbedGeometry : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolAutoEmbedGeometry(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual bool CanExecute() const override;

private:
	static TArray<UStaticMeshComponent*>  GetSelectedStaticMeshComponents();
};


UCLASS(DisplayName = "Flush Embedded Geometry", Category = "FractureTools")
class UFractureToolFlushEmbeddedGeometrySettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureToolFlushEmbeddedGeometrySettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, bExtractAsStaticMeshActors(false)
	{}

	UPROPERTY(EditAnywhere, Category = EmbeddedGeometry, meta = (DisplayName = "Extract As StaticMesh Actors"))
	bool bExtractAsStaticMeshActors;
};

UCLASS(DisplayName = "FlushEmbeddedGeometry", Category = "FractureTools")
class UFractureToolFlushEmbeddedGeometry : public UFractureModalTool
{
public:
	GENERATED_BODY()

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	UFractureToolFlushEmbeddedGeometry(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext);

	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	UPROPERTY(EditAnywhere, Category = EmbeddedGeometry)
	TObjectPtr<UFractureToolFlushEmbeddedGeometrySettings> FlushEmbeddedGeometrySettings;


};
