// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolFixTinyGeo.h"
#include "FractureEditorStyle.h"
#include "FractureSettings.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"
#include "Algo/RemoveIf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolFixTinyGeo)

#define LOCTEXT_NAMESPACE "FractureGeoMerge"


UFractureToolFixTinyGeo::UFractureToolFixTinyGeo(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	TinyGeoSettings = NewObject<UFractureTinyGeoSettings>(GetTransientPackage(), UFractureTinyGeoSettings::StaticClass());
	TinyGeoSettings->OwnerTool = this;
}

FText UFractureToolFixTinyGeo::GetDisplayText() const
{
	return FText(NSLOCTEXT("FixTinyGeo", "FractureToolFixTinyGeo", "Geometry Merge Tool")); 
}

FText UFractureToolFixTinyGeo::GetTooltipText() const 
{
	return FText(NSLOCTEXT("FixTinyGeo", "FractureToolFixTinyGeoTooltip", "The GeoMrg tool glues pieces of geometry onto their neighbors -- use it to, for example, clean up \"too small\" pieces of geometry."));
}

FSlateIcon UFractureToolFixTinyGeo::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.FixTinyGeo");
}

void UFractureToolFixTinyGeo::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "FixTinyGeo", "TinyGeo", "Merge pieces of geometry onto their neighbors -- use it to, for example, clean up \"too small\" pieces of geometry.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->FixTinyGeo = UICommandInfo;
}

void UFractureToolFixTinyGeo::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	EnumerateVisualizationMapping(ToRemoveMappings, ToRemoveBounds.Num(), [&](int32 Idx, FVector ExplodedVector)
	{
		const FBox& Box = ToRemoveBounds[Idx];
		FVector B000 = Box.Min + ExplodedVector;
		FVector B111 = Box.Max + ExplodedVector;
		FVector B011(B000.X, B111.Y, B111.Z);
		FVector B101(B111.X, B000.Y, B111.Z);
		FVector B110(B111.X, B111.Y, B000.Z);
		FVector B001(B000.X, B000.Y, B111.Z);
		FVector B010(B000.X, B111.Y, B000.Z);
		FVector B100(B111.X, B000.Y, B000.Z);
		PDI->DrawLine(B000, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B000, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B000, B001, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B110, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B001, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B001, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B110, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B110, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B100, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B010, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
	});
	
}

TArray<UObject*> UFractureToolFixTinyGeo::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(TinyGeoSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

bool UFractureToolFixTinyGeo::CollectTargetBones(FGeometryCollection& Collection, const TArray<int32>& Selection, TArray<int32>& OutSmallIndices, TArray<double>& OutVolumes, double& OutMinVolume)
{
	bool bClusterMode = TinyGeoSettings->MergeType == EMergeType::MergeClusters;
	bool bRestrictToLevel = bClusterMode && TinyGeoSettings->bOnFractureLevel;
	const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>();

	FindBoneVolumes(
		Collection,
		TArrayView<int32>(), /*Empty array => use all transforms*/
		OutVolumes,
		VolDimScale,
		bClusterMode
	);

	double TotalVolume = GetTotalVolume(Collection, OutVolumes);
	OutMinVolume = GetMinVolume(TotalVolume);

	TArray<int32> FilteredSelection;

	GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(Collection);

	if (TinyGeoSettings->UseBoneSelection == EUseBoneSelection::OnlyMergeSelected)
	{
		if (!bClusterMode)
		{
			OutSmallIndices = Selection;
			SelectionFacade.ConvertSelectionToRigidNodes(OutSmallIndices);
		}
		else
		{
			OutSmallIndices = Selection;
		}
		return !OutSmallIndices.IsEmpty();
	}

	FindSmallBones(
		Collection,
		TArrayView<int32>(),
		OutVolumes,
		OutMinVolume,
		OutSmallIndices,
		bClusterMode
	);

	// Filter bones that aren't at the target level
	int32 TargetLevel = FractureSettings->FractureLevel;
	const TManagedArray<int32>* LevelAttrib = Collection.FindAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	bool bFilterByLevel = bRestrictToLevel && TargetLevel > -1 && LevelAttrib;
	if (bFilterByLevel)
	{
		OutSmallIndices.SetNum(Algo::RemoveIf(OutSmallIndices, [&](int32 BoneIdx)
		{
			return (*LevelAttrib)[BoneIdx] != TargetLevel;
		}));
	}
	// Filter bones that aren't clusters
	if (bClusterMode && (!bFilterByLevel || TinyGeoSettings->bOnlyClusters))
	{
		OutSmallIndices.SetNum(Algo::RemoveIf(OutSmallIndices, [&](int32 BoneIdx)
		{
			return Collection.SimulationType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Clustered;
		}));
	}

	if (TinyGeoSettings->UseBoneSelection == EUseBoneSelection::AlsoMergeSelected)
	{
		TArray<int32> ProcessedSelection = Selection;
		if (!bClusterMode)
		{
			SelectionFacade.ConvertSelectionToRigidNodes(ProcessedSelection);
		}
		for (int32 Bone : ProcessedSelection)
		{
			OutSmallIndices.AddUnique(Bone);
		}
	}

	return !OutSmallIndices.IsEmpty();
}


void UFractureToolFixTinyGeo::FractureContextChanged()
{
	const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>();
	bool bLevelIsAll = FractureSettings->FractureLevel == -1;
	if (TinyGeoSettings->bFractureLevelIsAll != bLevelIsAll)
	{
		TinyGeoSettings->bFractureLevelIsAll = bLevelIsAll;
		NotifyOfPropertyChangeByTool(TinyGeoSettings);
	}

	UpdateDefaultRandomSeed();
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	ClearVisualizations();

	TArray<int32> Leaves;
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			// Get a non-const rest collection to support update of the Volume attribute
			FGeometryCollectionEdit EditCollection = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None, true);
			FGeometryCollection& Collection = *EditCollection.GetRestCollection()->GetGeometryCollection();
			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(Collection);

			TArray<int32> SelectedBones = GeometryCollectionComponent->GetSelectedBones();

			int CollectionIdx = VisualizedCollections.Emplace(GeometryCollectionComponent);

			TArray<int32> SmallIndices;
			TArray<double> Volumes;
			double MinVolume;
			CollectTargetBones(Collection, SelectedBones, SmallIndices, Volumes, MinVolume);

			FTransform OuterTransform = GeometryCollectionComponent->GetOwner()->GetActorTransform();
			for (int32 TransformIdx : SmallIndices) // small transforms
			{
				FTransform InnerTransform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TransformIdx);


				FTransform CombinedTransform = InnerTransform * OuterTransform;

				FBox Bounds(EForceInit::ForceInit);
				Leaves.Reset();
				SelectionFacade.ConvertSelectionToRigidNodes(TransformIdx, Leaves);
				for (int32 LeafIdx : Leaves)
				{
					int32 GeometryIdx = Collection.TransformToGeometryIndex[LeafIdx];
					if (GeometryIdx == INDEX_NONE)
					{
						ensure(false); // already filtered for small geo, shouldn't have any missing-geo transforms in the list
						continue;
					}
					int32 VStart = Collection.VertexStart[GeometryIdx];
					int32 VEnd = VStart + Collection.VertexCount[GeometryIdx];
					for (int32 VIdx = VStart; VIdx < VEnd; VIdx++)
					{
						Bounds += CombinedTransform.TransformPosition((FVector)Collection.Vertex[VIdx]);
					}
				}
				ToRemoveMappings.AddMapping(CollectionIdx, TransformIdx, ToRemoveBounds.Num());
				ToRemoveBounds.Add(Bounds);
			}
		}
	}
}

double UFractureToolFixTinyGeo::GetTotalVolume(const FGeometryCollection& Collection, const TArray<double>& Volumes)
{
	double Sum = 0.0;
	if (Volumes.Num() != Collection.Transform.Num())
	{
		return 0.0;
	}
	for (int32 BoneIdx = 0, NumBones = Collection.Transform.Num(); BoneIdx < NumBones; ++BoneIdx)
	{
		if (Collection.SimulationType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			Sum += Volumes[BoneIdx];
		}
	}
	return Sum;
}

double UFractureToolFixTinyGeo::GetMinVolume(double TotalVolume) const
{
	double MinVolume = 0;
	if (TinyGeoSettings->SelectionMethod == EGeometrySelectionMethod::VolumeCubeRoot)
	{
		MinVolume = TinyGeoSettings->MinVolumeCubeRoot * VolDimScale;
		MinVolume = MinVolume * MinVolume * MinVolume;
	}
	else // EGeometrySelectionMethod::RelativeVolume
	{
		MinVolume = FMath::Pow(TotalVolume, 1.0 / 3.0) * TinyGeoSettings->RelativeVolume;
		MinVolume = MinVolume * MinVolume * MinVolume;
	}
	return MinVolume;
}

void UFractureToolFixTinyGeo::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (!InToolkit.IsValid())
	{
		return;
	}
	FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			FGeometryCollectionEdit CollectionEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysicsDynamic, false);
			FGeometryCollection& Collection = *CollectionEdit.GetRestCollection()->GetGeometryCollection();
			TArray<int32> SelectedBones = GeometryCollectionComponent->GetSelectedBones();

			TArray<int32> SmallIndices;
			TArray<double> Volumes;
			double MinVolume;
			if (!CollectTargetBones(Collection, SelectedBones, SmallIndices, Volumes, MinVolume))
			{
				continue;
			}

			// convert ENeighborSelectionMethod to the PlanarCut module version (the only difference is it's not a UENUM)
			UE::PlanarCut::ENeighborSelectionMethod SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::LargestNeighbor;
			if (TinyGeoSettings->NeighborSelection == ENeighborSelectionMethod::NearestCenter)
			{
				SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::NearestCenter;
			}
			else if (TinyGeoSettings->NeighborSelection == ENeighborSelectionMethod::LargestContactArea)
			{
				SelectionMethod = UE::PlanarCut::ENeighborSelectionMethod::LargestContactArea;
			}

			if (TinyGeoSettings->MergeType == EMergeType::MergeGeometry)
			{
				MergeBones(
					Collection,
					TArrayView<const int32>(), // empty view == consider all bones
					Volumes,
					MinVolume,
					SmallIndices,
					false, /*bUnionJoinedPieces*/ // Note: Union-ing the pieces is nicer in theory, but can leave cracks and non-manifold garbage
					SelectionMethod,
					true /*bUseCollectionProximity*/,
					TinyGeoSettings->bOnlySameParent
				);
			}
			else
			{
				MergeClusters(
					Collection,
					Volumes,
					MinVolume,
					SmallIndices,
					SelectionMethod,
					TinyGeoSettings->bOnlyToConnected,
					TinyGeoSettings->bOnlySameParent
				);
			}

			// Refresh component with cleared selection
			TArray<int32> EmptySelection;
			Refresh(GeometryCollectionComponent, Toolkit, EmptySelection, true /*bClearSelection*/, true /*bMustUpdateBoneColors*/);
		}
	}

	// Update overall outliner (including stats, etc)
	Toolkit->SetOutlinerComponents(GeomCompSelection.Array());
}

int32 UFractureToolFixTinyGeo::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	// This tool instead overrides Execute, and should never call ExecuteFracture
	checkSlow(false);
	return INDEX_NONE;
}

void UFractureToolFixTinyGeo::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	if (InToolkit.IsValid())
	{
		InToolkit.Pin()->SetOutlinerColumnMode(EOutlinerColumnMode::Size);
	}
}


#undef LOCTEXT_NAMESPACE
