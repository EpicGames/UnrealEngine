// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "Engine/StaticMeshActor.h"

#include "FractureToolFixTinyGeo.generated.h"

class FFractureToolContext;

UENUM()
enum class EGeometrySelectionMethod
{
	// Select by cube root of volume
	VolumeCubeRoot UMETA(DisplayName = "Size"),
	// Select by cube root of volume relative to the overall shape's cube root of volume
	RelativeVolume
};

UENUM()
enum class ENeighborSelectionMethod
{
	// Merge to the neighbor with the largest volume
	LargestNeighbor,
	// Merge to the neighbor with the closest center
	NearestCenter,
	// Merge to the neighbor with the largest contact surface area
	LargestContactArea
};

UENUM()
enum class EUseBoneSelection
{
	NoEffect,
	AlsoMergeSelected,
	OnlyMergeSelected
};

UENUM()
enum class EMergeType : uint8
{
	MergeGeometry,
	MergeClusters
};

/** Settings controlling how geometry is selected and merged into neighboring geometry */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureTinyGeoSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureTinyGeoSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	// Whether to merge small geometry, or small clusters
	UPROPERTY(EditAnywhere, Category = MergeSettings)
	EMergeType MergeType = EMergeType::MergeGeometry;

	// Only consider bones at the current Fracture Level
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "!bFractureLevelIsAll && MergeType == EMergeType::MergeClusters", EditConditionHides))
	bool bOnFractureLevel = true;

	// Only auto-consider clusters for merging. Note that leaf nodes can still be consider if manually selected.
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "!bFractureLevelIsAll && MergeType == EMergeType::MergeClusters && bOnFractureLevel", EditConditionHides))
	bool bOnlyClusters = false;

	// Only merge to neighbors with the same parent in the hierarchy
	UPROPERTY(EditAnywhere, Category = MergeSettings)
	bool bOnlySameParent = true;

	// Helper variable to let the EditConditions above check whether the Fracture Level is set to 'All'
	UPROPERTY()
	bool bFractureLevelIsAll = false;

	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Merge To"))
	ENeighborSelectionMethod NeighborSelection = ENeighborSelectionMethod::LargestNeighbor;

	// Only merge pieces that are connected in the proximity graph. If unchecked, connected pieces will still be favored, but if none are available the closest disconnected piece can be merged.
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "MergeType == EMergeType::MergeClusters", EditConditionHides))
	bool bOnlyToConnected = true;

	/** Options for using the current bone selection */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Bone Selection"))
	EUseBoneSelection UseBoneSelection;


	UPROPERTY(EditAnywhere, Category = FilterSettings)
	EGeometrySelectionMethod SelectionMethod = EGeometrySelectionMethod::RelativeVolume;

	/** If size (cube root of volume) is less than this value, geometry should be merged into neighbors -- i.e. a value of 2 merges geometry smaller than a 2x2x2 cube */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (DisplayName = "MinSize", ClampMin = ".00001", UIMin = ".1", UIMax = "10", EditCondition = "SelectionMethod == EGeometrySelectionMethod::VolumeCubeRoot", EditConditionHides))
	double MinVolumeCubeRoot = 1;

	/** If cube root of volume relative to the overall shape's cube root of volume is less than this, the geometry should be merged into its neighbors.
	(Note: This is a bit different from the histogram viewer's "Relative Size," which instead shows values relative to the largest rigid bone.) */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = "0", UIMax = ".1", ClampMax = "1.0", EditCondition = "SelectionMethod == EGeometrySelectionMethod::RelativeVolume", EditConditionHides))
	double RelativeVolume = .01;

};



// Note this tool doesn't actually fracture, but it does remake pieces of geometry and shares a lot of machinery with the fracture tools
UCLASS(DisplayName = "Geometry Merge Tool", Category = "FractureTools")
class UFractureToolFixTinyGeo : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolFixTinyGeo(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureGeoMerge", "ExecuteGeoMerge", "Merge")); }

	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		ToRemoveBounds.Empty();
		ToRemoveMappings.Empty();
	}

private:

	UPROPERTY(EditAnywhere, Category = FixGeo)
	TObjectPtr<UFractureTinyGeoSettings> TinyGeoSettings;

	TArray<FBox> ToRemoveBounds; // Bounds in global space but without exploded vectors applied
	FVisualizationMappings ToRemoveMappings;

	static double GetTotalVolume(const FGeometryCollection& Collection, const TArray<double>& Volumes);
	double GetMinVolume(double TotalVolume) const;
	const double VolDimScale = .01; // compute volumes in meters instead of cm, for saner units at typical scales

	bool CollectTargetBones(FGeometryCollection& Collection, const TArray<int32>& Selection, TArray<int32>& OutSmallIndices, TArray<double>& OutVolumes, double& OutMinVolume);

};


