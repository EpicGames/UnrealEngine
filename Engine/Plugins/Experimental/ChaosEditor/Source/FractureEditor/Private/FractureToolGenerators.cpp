// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolGenerators.h"

#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Engine/Selection.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Layers/LayersSubsystem.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Change.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"

#include "FractureModeSettings.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConversion.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureToolContext.h"
#include "FractureToolBackgroundTask.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

#include "PhysicalMaterials/PhysicalMaterial.h"
#include "GeometryCollection/GeometryCollectionRootProxyRenderer.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/FileHelper.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"
#include "SPrimaryButton.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "PropertyCustomizationHelpers.h"

#include "Dataflow/DataflowObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolGenerators)

#define LOCTEXT_NAMESPACE "FractureToolGenerators"


namespace
{
	FString GetActiveContentBrowserFolderPath()
	{
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
		return CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : FString();
	}

}


//////////////////////////////////////////////////////////////////////////
// SCreateGeometryCollectionFromObject is adapted from SCreateAssetFromActor 

DECLARE_DELEGATE_OneParam(FOnPathChosen, const UFractureToolGenerateAsset::FCreationParameters& Params);

class SCreateGeometryCollectionFromObject : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCreateGeometryCollectionFromObject)
		: _AssetFilenameSuffix()
		, _AssetFilenamePrefix()
		, _HeadingText()
		, _CreateButtonText()
		, _bSplitIslands(false)
		, _bReimportToMeshOutput(false)
		, _DataflowAsset(nullptr)
		, _PhysicalMaterial(nullptr)
	{}

	/** The default suffix to use for the asset filename */
	SLATE_ARGUMENT(FString, AssetFilenameSuffix)

	/** The default suffix to use for the asset filename */
	SLATE_ARGUMENT(FString, AssetFilenamePrefix)

	/** The text to display at the top of the dialog */
	SLATE_ARGUMENT(FText, HeadingText)

	/** The label for the create button */
	SLATE_ARGUMENT(FText, CreateButtonText)

	/** The path we begin naviagation at */
	SLATE_ARGUMENT(FString, AssetPath)

	SLATE_ARGUMENT(FText, DefaultNameOverride)

	SLATE_ARGUMENT(bool, bSplitIslands)
	SLATE_ARGUMENT(bool, bReimportToMeshOutput)
	SLATE_ARGUMENT(bool, bUseRootProxies)
	SLATE_ARGUMENT(UDataflow*, DataflowAsset)
	SLATE_ARGUMENT(UPhysicalMaterial*, PhysicalMaterial)

	/** Action to perform when create clicked */
	SLATE_EVENT(FOnPathChosen, OnCreateAssetAction)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow);

private:

	/** Callback when the "create asset" button is clicked. */
	FReply OnCreateAssetFromActorClicked();

	/** Callback when the selected asset path has changed. */
	void OnSelectAssetPath(const FString& Path);

	FString GetDataflowAssetPath() const
	{
		if (DataflowAsset)
		{
			return DataflowAsset->GetPathName();
		}
		return FString();
	}
	void OnDataflowAssetChanged(const FAssetData& InAssetData)
	{
		DataflowAsset = Cast<UDataflow>(InAssetData.GetAsset());
	}

	FString GetPhysicalMaterialPath() const
	{
		if (PhysicalMaterial)
		{
			return PhysicalMaterial->GetPathName();
		}
		return FString();
	}
	void OnPhysicalMaterialChanged(const FAssetData& InAssetData)
	{
		PhysicalMaterial = Cast<UPhysicalMaterial>(InAssetData.GetAsset());
	}
	

	/** Destroys the window when the operation is cancelled. */
	FReply OnCancelCreateAssetFromActor();

	/** Callback when level selection has changed, will destroy the window. */
	void OnLevelSelectionChanged(UObject* InObjectSelected);

	/** Callback when the user changes the filename for the Blueprint */
	void OnFilenameChanged(const FText& InNewName);

	/** Callback to see if creating an asset is enabled */
	bool IsCreateAssetFromActorEnabled() const;

	/** Rquest to destroy the parent window */
	void RequestDestroyParentWindow();

private:
	/** The window this widget is nested in */
	TWeakPtr<SWindow> ParentWindow;

	/** The selected path to create the asset */
	FString AssetPath;

	/** The resultant actor instance label, based on the original actor labels */
	FString ActorInstanceLabel;

	/** The default suffix to use for the asset filename */
	FString AssetFilenameSuffix;

	/** The default prefix to use for the asset filename */
	FString AssetFilenamePrefix;

	/** The text to display as a heading for the dialog */
	FText HeadingText;

	/** The label to be displayed on the create button */
	FText CreateButtonText;

	/** Whether to split out topologically-disconnected components into separate bones in the Geometry Collection */
	bool bSplitIslands = false;

	/** Designate input faces with odd-numbered material indices as 'internal' materials. Only use this option to preserve internal faces on meshes created with the Fracture Mode 'ToMesh' tool. */
	bool bFromToMeshTool = false;

	/** Whether to use the meshes used to create the geometry collection as root proxies */
	bool bUseRootProxies = true;

	/** Dataflow asset to assign */
	TObjectPtr<UDataflow> DataflowAsset = nullptr;

	/** Physical Material to assign */
	TObjectPtr<UPhysicalMaterial> PhysicalMaterial = nullptr;

	/** Filename textbox widget */
	TSharedPtr<SEditableTextBox> FileNameWidget;

	/** True if an error is currently being reported */
	bool bIsReportingError;

	/** Called when the create button is clicked */
	FOnPathChosen OnCreateAssetAction;

	/** Used to keep track of the delegate in the selection event */
	FDelegateHandle SelectionDelegateHandle;
};



void SCreateGeometryCollectionFromObject::Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow)
{
	AssetFilenameSuffix = InArgs._AssetFilenameSuffix;
	AssetFilenamePrefix = InArgs._AssetFilenamePrefix;
	HeadingText = InArgs._HeadingText;
	CreateButtonText = InArgs._CreateButtonText;
	OnCreateAssetAction = InArgs._OnCreateAssetAction;
	bSplitIslands = InArgs._bSplitIslands;
	bFromToMeshTool = InArgs._bReimportToMeshOutput;
	bUseRootProxies = InArgs._bUseRootProxies;
	DataflowAsset = InArgs._DataflowAsset;
	PhysicalMaterial = InArgs._PhysicalMaterial;


	if (InArgs._AssetPath.IsEmpty())
	{
		AssetPath = FString("/Game");
	}
	else
	{
		AssetPath = InArgs._AssetPath;
	}

	bIsReportingError = false;

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.bShowViewOptions = true;
	PathPickerConfig.DefaultPath = AssetPath;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateRaw(this, &SCreateGeometryCollectionFromObject::OnSelectAssetPath);

	SelectionDelegateHandle = USelection::SelectionChangedEvent.AddSP(this, &SCreateGeometryCollectionFromObject::OnLevelSelectionChanged);

	// Set up PathPickerConfig.
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	ParentWindow = InParentWindow;

	FString PackageName;
	ActorInstanceLabel.Empty();

	if (InArgs._DefaultNameOverride.IsEmpty())
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			if (Actor)
			{
				ActorInstanceLabel += Actor->GetActorLabel();
				break;
			}
		}
	}
	else
	{
		ActorInstanceLabel = InArgs._DefaultNameOverride.ToString();
	}

	if (!AssetFilenamePrefix.IsEmpty())
	{
		ActorInstanceLabel = AssetFilenamePrefix + TEXT("_") + ActorInstanceLabel;
	}
	if (!AssetFilenameSuffix.IsEmpty())
	{
		ActorInstanceLabel = ActorInstanceLabel + TEXT("_") + AssetFilenameSuffix;
	}

	ActorInstanceLabel = UPackageTools::SanitizePackageName(ActorInstanceLabel);

	FString AssetName = ActorInstanceLabel;
	FString BasePath = AssetPath / AssetName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, AssetName);

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 8.0f, 0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(HeadingText)
		]

	+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(FileNameWidget, SEditableTextBox)
			.Text(FText::FromString(AssetName))
		.OnTextChanged(this, &SCreateGeometryCollectionFromObject::OnFilenameChanged)
		]
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(4.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.FillWidth(0.5f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ReimportToMeshLabel", "Reimport From 'ToMesh'"))
						.ToolTipText(LOCTEXT("FromToMeshToolTip", "Only check for meshes that were created by the ToMesh tool in Fracture Mode. Does not apply if creating from a Geometry Collection source."))
					]

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()->ECheckBoxState
							{
								return bFromToMeshTool ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
							{
								if (NewState == ECheckBoxState::Undetermined)
								{
									return;
								}
								bFromToMeshTool = NewState == ECheckBoxState::Checked;
							})
						.ToolTipText(LOCTEXT("FromToMeshToolTip", "Only check for meshes that were created by the ToMesh tool in Fracture Mode. Does not apply if creating from a Geometry Collection source."))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.FillWidth(0.5f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SplitMeshesLabel", "Split Meshes"))
						.ToolTipText(LOCTEXT("SplitComponentsToolTip", "If checked, triangles that are not topologically connected will be assigned separate bones on import. Does not apply if creating from a Geometry Collection source."))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()->ECheckBoxState
							{
								return bSplitIslands ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
						{
							if (NewState == ECheckBoxState::Undetermined)
							{
								return;
							}
							bSplitIslands = NewState == ECheckBoxState::Checked;
						})
						.ToolTipText(LOCTEXT("SplitComponentsToolTip", "If checked, triangles that are not topologically connected will be assigned separate bones on import. Does not apply if creating from a Geometry Collection source."))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.FillWidth(0.5f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UseRootProxiesLabel", "Use Root Proxies"))
						.ToolTipText(LOCTEXT("UseRootProxiesToolTip", "If checked, the meshes used to create the geometry collection will be used as root proxies for rendering the unbroken state."))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()->ECheckBoxState
							{
								return bUseRootProxies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
						{
							if (NewState == ECheckBoxState::Undetermined)
							{
								return;
							}
							bUseRootProxies = NewState == ECheckBoxState::Checked;
						})
						.ToolTipText(LOCTEXT("UseRootProxiesToolTip", "If checked, the meshes used to create the geometry collection will be used as root proxies for rendering the unbroken state."))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.FillWidth(0.5f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DataflowAssetLabel", "Dataflow"))
						.ToolTipText(LOCTEXT("DataflowAssetToolTip", "Optionally assign a Dataflow to the new Geometry Collection, to procedurally define its fracture and clustering."))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SObjectPropertyEntryBox)
						.ObjectPath(this, &SCreateGeometryCollectionFromObject::GetDataflowAssetPath)
						.OnObjectChanged(this, &SCreateGeometryCollectionFromObject::OnDataflowAssetChanged)
						.AllowedClass(UDataflow::StaticClass())
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.AllowCreate(true)
						.AllowClear(true)
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.FillWidth(0.5f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PhysicalMaterialAssetLabel", "PhysicalMaterial"))
						.ToolTipText(LOCTEXT("PhysicalMaterialAssetToolTip", "Optionally assign a Physical material to the new Geometry Collection."))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SObjectPropertyEntryBox)
						.ObjectPath(this, &SCreateGeometryCollectionFromObject::GetPhysicalMaterialPath)
						.OnObjectChanged(this, &SCreateGeometryCollectionFromObject::OnPhysicalMaterialChanged)
						.AllowedClass(UPhysicalMaterial::StaticClass())
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.AllowCreate(true)
						.AllowClear(true)
					]
				]
			]
		]

	+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		.Padding(0.f, 16.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SPrimaryButton)
			.OnClicked(this, &SCreateGeometryCollectionFromObject::OnCreateAssetFromActorClicked)
		.Text(CreateButtonText)
		]

	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(16.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("CancelButtonText", "Cancel"))
		.OnClicked(this, &SCreateGeometryCollectionFromObject::OnCancelCreateAssetFromActor)
		]
		]
		]
		]
		];

	OnFilenameChanged(FText::FromString(AssetName));
}

void SCreateGeometryCollectionFromObject::RequestDestroyParentWindow()
{
	USelection::SelectionChangedEvent.Remove(SelectionDelegateHandle);

	if (ParentWindow.IsValid())
	{
		ParentWindow.Pin()->RequestDestroyWindow();
	}
}

FReply SCreateGeometryCollectionFromObject::OnCreateAssetFromActorClicked()
{
	RequestDestroyParentWindow();
	UFractureToolGenerateAsset::FCreationParameters Params
	{
		.AssetPath = AssetPath / FileNameWidget->GetText().ToString(),
		.bFromToMeshTool = bFromToMeshTool,
		.bSplitIslands = bSplitIslands,
		.bUseRootProxies = bUseRootProxies,
		.DataflowAsset = DataflowAsset,
		.PhysicalMaterial = PhysicalMaterial,
	};
	OnCreateAssetAction.ExecuteIfBound(Params);
	return FReply::Handled();
}

FReply SCreateGeometryCollectionFromObject::OnCancelCreateAssetFromActor()
{
	RequestDestroyParentWindow();
	return FReply::Handled();
}

void SCreateGeometryCollectionFromObject::OnSelectAssetPath(const FString& Path)
{
	AssetPath = Path;
	OnFilenameChanged(FileNameWidget->GetText());
}

void SCreateGeometryCollectionFromObject::OnLevelSelectionChanged(UObject* InObjectSelected)
{
	// When actor selection changes, this window should be destroyed.
	RequestDestroyParentWindow();
}

void SCreateGeometryCollectionFromObject::OnFilenameChanged(const FText& InNewName)
{
	TArray<FAssetData> AssetData;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().GetAssetsByPath(FName(*AssetPath), AssetData);

	FText ErrorText;
	if (!FFileHelper::IsFilenameValidForSaving(InNewName.ToString(), ErrorText) || !FName(*InNewName.ToString()).IsValidObjectName(ErrorText))
	{
		FileNameWidget->SetError(ErrorText);
		bIsReportingError = true;
		return;
	}
	else
	{
		// Check to see if the name conflicts
		for (auto Iter = AssetData.CreateConstIterator(); Iter; ++Iter)
		{
			if (Iter->AssetName.ToString() == InNewName.ToString())
			{
				FileNameWidget->SetError(LOCTEXT("AssetInUseError", "Asset name already in use!"));
				bIsReportingError = true;
				return;
			}
		}
	}

	FileNameWidget->SetError(FText::FromString(TEXT("")));
	bIsReportingError = false;
}

bool SCreateGeometryCollectionFromObject::IsCreateAssetFromActorEnabled() const
{
	return !bIsReportingError;
}

FText UFractureToolGenerateAsset::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolGenerateAsset", "New"));
}

FText UFractureToolGenerateAsset::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolGenerateAssetTooltip", "Generate a Geometry Collection Asset from selected Static Meshes and/or Geometry Collections."));
}

FSlateIcon UFractureToolGenerateAsset::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.GenerateAsset");
}

void UFractureToolGenerateAsset::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "GenerateAsset", "New", "Generate a new Geometry Collection Asset from the selected Static Meshes and/or Geometry Collections. Geometry Collections are assets that support fracture.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->GenerateAsset = UICommandInfo;
}

bool UFractureToolGenerateAsset::CanExecute() const
{
	return (IsStaticMeshSelected() || IsGeometryCollectionSelected());
}

void UFractureToolGenerateAsset::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Toolkit = InToolkit;
	
	USelection* SelectionSet = GEditor->GetSelectedActors();

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());

	SelectionSet->GetSelectedObjects(SelectedActors);

	OpenGenerateAssetDialog(SelectedActors);

	// Note: Transaction for undo history is created after the user completes the UI dialog; see OnGenerateAssetPathChosen()
}

FString UFractureToolGenerateAsset::GetDefaultAssetPath(const TArray<AActor*>& Actors) const
{
	FString UseAssetPath;

	const UFractureModeSettings* Settings = GetDefault<UFractureModeSettings>();
	bool bFailedToFindContentBrowser = false, bFailedToFindLastUsedFolder = false;
	if (Settings->NewAssetLocation == EFractureModeNewAssetLocation::ContentBrowserFolder)
	{
		FString ActiveContentFolder = GetActiveContentBrowserFolderPath();
		if (ActiveContentFolder.IsEmpty())
		{
			bFailedToFindContentBrowser = true;
		}
		else
		{
			UseAssetPath = ActiveContentFolder;
		}
	}
	if (bFailedToFindContentBrowser || Settings->NewAssetLocation == EFractureModeNewAssetLocation::LastUsedFolder)
	{
		if (AssetPath.IsEmpty())
		{
			bFailedToFindLastUsedFolder = true;
		}
		else
		{
			UseAssetPath = AssetPath;
		}
	}
	if (bFailedToFindLastUsedFolder || Settings->NewAssetLocation == EFractureModeNewAssetLocation::SourceAssetFolder)
	{
		for (const AActor* Actor : Actors)
		{
			// Find the asset folder of any selected static mesh
			TArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents, true);
			const UStaticMesh* FoundMeshAsset = nullptr;
			for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
			{
				FoundMeshAsset = StaticMeshComponent ? StaticMeshComponent->GetStaticMesh().Get() : nullptr;
				if (FoundMeshAsset)
				{
					break;
				}
			}
			if (FoundMeshAsset)
			{
				UseAssetPath = FoundMeshAsset->GetPathName();
				break;
			}
			// Find the asset folder of any selected geometry collection
			TArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
			Actor->GetComponents(GeometryCollectionComponents, true);
			const UGeometryCollection* FoundRestAsset = nullptr;
			for (UGeometryCollectionComponent* GCComp : GeometryCollectionComponents)
			{
				FoundRestAsset = GCComp ? GCComp->GetRestCollection() : nullptr;
				if (FoundRestAsset)
				{
					break;
				}
			}
			if (FoundRestAsset)
			{
				UseAssetPath = FoundRestAsset->GetPathName();
				break;
			}
		}
	}
	return UseAssetPath;
}


void UFractureToolGenerateAsset::OpenGenerateAssetDialog(TArray<AActor*>& Actors)
{
	TSharedPtr<SWindow> PickAssetPathWindow;

	SAssignNew(PickAssetPathWindow, SWindow)
		.Title(LOCTEXT("SelectPath", "Select Path"))
		.ToolTipText(LOCTEXT("SelectPathTooltip", "Select the asset path for your new Geometry Collection"))
		.ClientSize(FVector2D(500, 500));

	AssetPath = GetDefaultAssetPath(Actors);

	// NOTE - the parent window has to completely exist before this one does so the parent gets set properly.
	// This is why we do not just put this in the Contents()[ ... ] of the Window above.
	TSharedPtr<SCreateGeometryCollectionFromObject> CreateAssetDialog;
	PickAssetPathWindow->SetContent(
		SAssignNew(CreateAssetDialog, SCreateGeometryCollectionFromObject, PickAssetPathWindow)
		.AssetFilenamePrefix(TEXT("GC"))
		.HeadingText(LOCTEXT("CreateGeometryCollection_Heading", "Geometry Collection Name"))
		.CreateButtonText(LOCTEXT("CreateGeometryCollection_ButtonLabel", "Create Geometry Collection"))
		.AssetPath(AssetPath)
		.DataflowAsset(LastDataflowAsset)
		.PhysicalMaterial(LastPhysicalMaterial)
		.bReimportToMeshOutput(bLastFromToMesh)
		.bSplitIslands(bLastSplitComponents)
		.bUseRootProxies(bLastUseRootProxies)
		.OnCreateAssetAction(FOnPathChosen::CreateUObject(this, &UFractureToolGenerateAsset::OnGenerateAssetPathChosen, Actors))
	);

	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	if (RootWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(PickAssetPathWindow.ToSharedRef(), RootWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(PickAssetPathWindow.ToSharedRef());
	}

}

void UFractureToolGenerateAsset::OnGenerateAssetPathChosen(const UFractureToolGenerateAsset::FCreationParameters& Params, TArray<AActor*> Actors)
{	
	//Record the path and other settings
	int32 LastSlash = INDEX_NONE;
	if (Params.AssetPath.FindLastChar('/', LastSlash))
	{
		AssetPath = Params.AssetPath.Left(LastSlash);
	}
	bLastFromToMesh = Params.bFromToMeshTool;
	bLastSplitComponents = Params.bSplitIslands;
	bLastUseRootProxies= Params.bUseRootProxies;
	LastDataflowAsset = Params.DataflowAsset;
	LastPhysicalMaterial = Params.PhysicalMaterial;

	UGeometryCollectionComponent* GeometryCollectionComponent = nullptr;

	if (Actors.Num() > 0)
	{
		AActor* FirstActor = Actors[0];

		AGeometryCollectionActor* GeometryCollectionActor = nullptr;
		
		GeometryCollectionActor = ConvertActorsToGeometryCollection(Params, false/*bAddInternalMaterials*/, Actors);
		if (!GeometryCollectionActor)
		{
			return;
		}

		GeometryCollectionComponent = GeometryCollectionActor->GetGeometryCollectionComponent();

		FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();
		EditBoneColor.SetShowBoneColors(true);

		// Move GC actor to source actors position and remove source actor from scene
		const FVector ActorLocation(FirstActor->GetActorLocation());
		GeometryCollectionActor->SetActorLocation(ActorLocation);

		// Clear selection of mesh actor used to make GC before selecting, will cause details pane to not display geometry collection details.
		GEditor->SelectNone(true, true, false);
		GEditor->SelectActor(GeometryCollectionActor, true, true);

		EditBoneColor.SelectBones(GeometryCollection::ESelectionMode::AllGeometry);

		if (Toolkit.IsValid())
		{
			TSharedPtr<FFractureEditorModeToolkit> SharedToolkit(Toolkit.Pin());
			SharedToolkit->SetOutlinerComponents({ GeometryCollectionComponent });
			SharedToolkit->SetBoneSelection(GeometryCollectionComponent, EditBoneColor.GetSelectedBones(), true);

			SharedToolkit->OnSetLevelViewValue(-1);

			SharedToolkit->RegenerateOutliner();
			SharedToolkit->RegenerateHistogram();

			SharedToolkit->UpdateExplodedVectors(GeometryCollectionComponent);
		}
		
		GeometryCollectionComponent->MarkRenderDynamicDataDirty();
		GeometryCollectionComponent->MarkRenderStateDirty();



		FScopedTransaction Transaction(LOCTEXT("RemoveSourceActors", "Remove Source Actors"));

		auto DestroyActor = [](AActor* Actor)
		{
			if (UWorld* ActorWorld = Actor->GetWorld())
			{
				if (GIsEditor && GUnrealEd)
				{
					GUnrealEd->DeleteActors(TArray{ Actor }, ActorWorld, GUnrealEd->GetSelectedActors()->GetElementSelectionSet());
				}
				else
				{
					ActorWorld->DestroyActor(Actor);
				}
			}
		};

		for (AActor* Actor : Actors)
		{
			Actor->Modify();
			DestroyActor(Actor);
		}
	}
}

AGeometryCollectionActor* UFractureToolGenerateAsset::ConvertActorsToGeometryCollection(const FCreationParameters& Params, bool bAddInternalMaterials, TArray<AActor*>& Actors)
{
	ensure(!bAddInternalMaterials); // we should not use the 'add internal materials' path anymore, as we move away from the odd-numbered-internal-material convention
	ensure(Actors.Num() > 0);
	AActor* FirstActor = Actors[0];
	const FString& Name = FirstActor->GetActorLabel();
	const FVector FirstActorLocation(FirstActor->GetActorLocation());

	using FGeometryCollectionSharedPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	// Count the total triangles of all input sources
	int32 SourceTriCount = 0;
	{
		for (AActor* Actor : Actors)
		{
			TArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents, true);
			for (int32 ii = 0, ni = StaticMeshComponents.Num(); ii < ni; ++ii)
			{
				if (UStaticMeshComponent* StaticMeshComponent = StaticMeshComponents[ii])
				{
					if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
					{
						SourceTriCount += StaticMesh->GetNumTriangles(0);
					}
				}
			}

			TArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
			Actor->GetComponents(GeometryCollectionComponents, true);
			for (int32 ii = 0, ni = GeometryCollectionComponents.Num(); ii < ni; ++ii)
			{
				if (UGeometryCollectionComponent* GeometryCollectionComponent = GeometryCollectionComponents[ii])
				{
					if (const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection())
					{
						if (const FGeometryCollectionSharedPtr Collection = RestCollection->GetGeometryCollection())
						{
							SourceTriCount += Collection->Indices.Num();
						}
					}
				}
			}
		}
	}
	constexpr int32 ThresholdToCautionAboutLargeInput = 1000000;
	if (SourceTriCount > ThresholdToCautionAboutLargeInput)
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
			FText::Format(LOCTEXT("FractureLargeInputQuestion", "Sources with a large number of triangles ({0}) may take time to convert to Geometry Collection. Continue?"), SourceTriCount),
			LOCTEXT("FractureLargeInputTitle", "Process large input?"));
		if (Ret == EAppReturnType::No)
		{
			return nullptr;
		}
	}

	FScopedSlowTask SlowTask(SourceTriCount, LOCTEXT("FractureCreateNewGeometryCollection", "Creating new Geometry Collection"));
	SlowTask.MakeDialog();

	AGeometryCollectionActor* NewActor = CreateNewGeometryActor(Params.AssetPath, FTransform(), true);

	FGeometryCollectionEdit GeometryCollectionEdit = NewActor->GetGeometryCollectionComponent()->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysicsDynamic);
	UGeometryCollection* FracturedGeometryCollection = GeometryCollectionEdit.GetRestCollection();
	check(FracturedGeometryCollection);

	for (AActor* Actor : Actors)
	{
		const FTransform ActorTransform(Actor->GetTransform());
		const FVector ActorOffset(Actor->GetActorLocation() - FirstActor->GetActorLocation());

		TArray<UStaticMeshComponent*> StaticMeshComponents;
		Actor->GetComponents(StaticMeshComponents, true);
		for (int32 ii = 0, ni = StaticMeshComponents.Num(); ii < ni; ++ii)
		{
			// We're partial to static mesh components, here
			UStaticMeshComponent* StaticMeshComponent = StaticMeshComponents[ii];
			if (StaticMeshComponent != nullptr)
			{
				UStaticMesh* ComponentStaticMesh = StaticMeshComponent->GetStaticMesh();
				if (ComponentStaticMesh != nullptr)
				{
					// If any of the static meshes have Nanite enabled, also enable on the new geometry collection asset for convenience.
					FracturedGeometryCollection->EnableNanite |= ComponentStaticMesh->IsNaniteEnabled();
					SlowTask.EnterProgressFrame(ComponentStaticMesh->GetNumTriangles(0));
				}

				FTransform ComponentTransform(StaticMeshComponent->GetComponentTransform());
				ComponentTransform.SetTranslation((ComponentTransform.GetTranslation() - ActorTransform.GetTranslation()) + ActorOffset);

				// Record the contributing source on the asset.
				FSoftObjectPath SourceSoftObjectPath(ComponentStaticMesh);
				decltype(FGeometryCollectionSource::SourceMaterial) SourceMaterials(StaticMeshComponent->GetMaterials());
				bool bSetInternalFromMaterialIndex = Params.bFromToMeshTool;
				
				FracturedGeometryCollection->GeometrySource.Emplace(SourceSoftObjectPath, ComponentTransform, SourceMaterials, Params.bSplitIslands, bSetInternalFromMaterialIndex);
				
				FGeometryCollectionEngineConversion::AppendStaticMesh(ComponentStaticMesh, SourceMaterials, ComponentTransform, FracturedGeometryCollection, false/*bReindexMaterials*/, bAddInternalMaterials, Params.bSplitIslands, bSetInternalFromMaterialIndex);
			}
		}

		TArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
		Actor->GetComponents(GeometryCollectionComponents, true);
		for (int32 ii = 0, ni = GeometryCollectionComponents.Num(); ii < ni; ++ii)
		{
			UGeometryCollectionComponent* GeometryCollectionComponent = GeometryCollectionComponents[ii];
			if (GeometryCollectionComponent != nullptr)
			{
				const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection();
				if (RestCollection != nullptr)
				{
					// If any of the static meshes have Nanite enabled, also enable on the new geometry collection asset for convenience.
					FracturedGeometryCollection->EnableNanite |= RestCollection->EnableNanite;

					if (const FGeometryCollectionSharedPtr Collection = RestCollection->GetGeometryCollection())
					{
						SlowTask.EnterProgressFrame(Collection->Indices.Num());
					}
				}

				FTransform ComponentTransform(GeometryCollectionComponent->GetComponentTransform());
				ComponentTransform.SetTranslation((ComponentTransform.GetTranslation() - ActorTransform.GetTranslation()) + ActorOffset);

				// Record the contributing source on the asset.
				FSoftObjectPath SourceSoftObjectPath(RestCollection);

				int32 NumMaterials = GeometryCollectionComponent->GetNumMaterials();
				TArray<TObjectPtr<UMaterialInterface>> SourceMaterials;
				SourceMaterials.SetNum(NumMaterials);
				for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
				{
					SourceMaterials[MaterialIndex] = GeometryCollectionComponent->GetMaterial(MaterialIndex);
				}
				constexpr bool bSplitCollectionComponents = false, bSetInternalFromMaterialIndex = false;
				FracturedGeometryCollection->GeometrySource.Emplace(SourceSoftObjectPath, ComponentTransform, SourceMaterials, bSplitCollectionComponents, bSetInternalFromMaterialIndex);

				FGeometryCollectionEngineConversion::AppendGeometryCollection(RestCollection, GeometryCollectionComponent, ComponentTransform, FracturedGeometryCollection, false /*bReindexMaterials*/);

			}
		}
	}

	if (Params.PhysicalMaterial)
	{
		FracturedGeometryCollection->PhysicsMaterial = Params.PhysicalMaterial;
	}

	if (Params.bUseRootProxies)
	{
		FracturedGeometryCollection->CustomRendererType = UGeometryCollectionRootProxyRenderer::StaticClass();
		FracturedGeometryCollection->SetRootProxiesFromGeometrySources();
	}

	FracturedGeometryCollection->InitializeMaterials(bAddInternalMaterials);

	if (Params.DataflowAsset)
	{
		FracturedGeometryCollection->DataflowInstance.SetDataflowAsset(Params.DataflowAsset);
	}

	AddSingleRootNodeIfRequired(FracturedGeometryCollection);

	FracturedGeometryCollection->InvalidateCollection();
	FracturedGeometryCollection->RebuildRenderData();

	// Add and initialize guids
	::GeometryCollection::GenerateTemporaryGuids(FracturedGeometryCollection->GetGeometryCollection().Get(), 0 , true);

	const UFractureModeSettings* ModeSettings = GetDefault<UFractureModeSettings>();
	ModeSettings->ApplyDefaultSettings(*FracturedGeometryCollection->GetGeometryCollection());

	return NewActor;
}

class AGeometryCollectionActor* UFractureToolGenerateAsset::CreateNewGeometryActor(const FString& InAssetPath, const FTransform& Transform, bool AddMaterials /*= false*/)
{
	FString UniquePackageName = InAssetPath;
	FString UniqueAssetName = FPackageName::GetLongPackageAssetName(InAssetPath);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(UniquePackageName, TEXT(""), UniquePackageName, UniqueAssetName);

	UPackage* Package = CreatePackage(*UniquePackageName);
	UGeometryCollection* InGeometryCollection = static_cast<UGeometryCollection*>(NewObject<UGeometryCollection>(Package, UGeometryCollection::StaticClass(), FName(*UniqueAssetName), RF_Transactional | RF_Public | RF_Standalone));

	// Create the new Geometry Collection actor
	ULevel* PlaceInLevel = GEditor->GetEditorWorldContext().World()->GetCurrentLevel();
	AGeometryCollectionActor* NewActor = Cast<AGeometryCollectionActor>(AddActor(PlaceInLevel, AGeometryCollectionActor::StaticClass()));
	check(NewActor->GetGeometryCollectionComponent());

	// Set the Geometry Collection asset in the new actor
	NewActor->GetGeometryCollectionComponent()->SetRestCollection(InGeometryCollection);
	if (InGeometryCollection->PhysicsMaterial == nullptr)
	{
		NewActor->GetGeometryCollectionComponent()->SetPhysMaterialOverride(GEngine->DefaultDestructiblePhysMaterial);
	}

	// copy transform of original static mesh actor to this new actor
	NewActor->SetActorLabel(UniqueAssetName);
	NewActor->SetActorTransform(Transform);

	// Mark relevant stuff dirty
	FAssetRegistryModule::AssetCreated(InGeometryCollection);
	InGeometryCollection->MarkPackageDirty();
	Package->SetDirtyFlag(true);

	return NewActor;
}

AActor* UFractureToolGenerateAsset::AddActor(ULevel* InLevel, UClass* Class)
{
	check(Class);

	UWorld* World = InLevel->OwningWorld;
	ULevel* DesiredLevel = InLevel;

	// Transactionally add the actor.
	AActor* Actor = NULL;
	{
		FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "AddActor", "Add Actor"));

		FActorSpawnParameters SpawnInfo;
		SpawnInfo.OverrideLevel = DesiredLevel;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.ObjectFlags = RF_Transactional;
		const auto Location = FVector(0);
		const auto Rotation = FTransform(FVector(0)).GetRotation().Rotator();
		Actor = World->SpawnActor(Class, &Location, &Rotation, SpawnInfo);

		check(Actor);
		Actor->InvalidateLightingCache();
		Actor->PostEditMove(true);
	}

	// If this actor is part of any layers (set in its default properties), add them into the visible layers list.
	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	Layers->SetLayersVisibility(Actor->Layers, true);

	// Clean up.
	Actor->MarkPackageDirty();
	ULevel::LevelDirtiedEvent.Broadcast();

	return Actor;
}


UFractureToolResetAsset::UFractureToolResetAsset(const FObjectInitializer& ObjInit) : Super(ObjInit)
{
	ResetSettings = NewObject<UGeometryCollectionResetSettings>(GetTransientPackage(), UGeometryCollectionResetSettings::StaticClass());
	ResetSettings->OwnerTool = this;
}

FText UFractureToolResetAsset::GetApplyText() const
{ 
	return FText(NSLOCTEXT("FractureToolReset", "FractureToolResetAction", "Reset")); 
}

TArray<UObject*> UFractureToolResetAsset::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ResetSettings);
	return Settings;
}


FText UFractureToolResetAsset::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolReset", "Reset"));
}

FText UFractureToolResetAsset::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolResetTooltip", "Reset Geometry Collections to their initial unfractured states."));
}

FSlateIcon UFractureToolResetAsset::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.ResetAsset");
}

void UFractureToolResetAsset::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "ResetAsset", "Reset", "Reset selected Geometry Collections to their initial unfractured states.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->ResetAsset = UICommandInfo;
}

bool UFractureToolResetAsset::CanExecute() const
{
	return IsGeometryCollectionSelected();
}

void UFractureToolResetAsset::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (!InToolkit.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetCollection", "Reset Geometry Collection"));

	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		GeometryCollectionComponent->Modify();
		FGeometryCollectionEdit GeometryCollectionEdit = GeometryCollectionComponent->EditRestCollection();
		if (UGeometryCollection* GeometryCollectionObject = GeometryCollectionEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				bool bKeepPreviousMaterials = !ResetSettings->bResetMaterials;
				TArray<TObjectPtr<UMaterialInterface>> OldMaterials;
				if (bKeepPreviousMaterials)
				{
					OldMaterials = GeometryCollectionObject->Materials;
				}

				GeometryCollectionObject->Reset();

				// Rebuild Collection from recorded source assets.
				for (const FGeometryCollectionSource& Source : GeometryCollectionObject->GeometrySource)
				{
					const UObject* SourceMesh = Source.SourceGeometryObject.TryLoad();
					if (const UStaticMesh* SourceStaticMesh = Cast<UStaticMesh>(SourceMesh))
					{
						bool bLegacyAddInternal = Source.bAddInternalMaterials;
						FGeometryCollectionEngineConversion::AppendStaticMesh(SourceStaticMesh, Source.SourceMaterial, Source.LocalTransform, GeometryCollectionObject, false, bLegacyAddInternal, Source.bSplitComponents, Source.bSetInternalFromMaterialIndex);
					}
					else if (const USkeletalMesh* SourceSkeletalMesh = Cast<USkeletalMesh>(SourceMesh))
					{
						// #todo (bmiller) Once we've settled on the right approach with static meshes, we'll need to apply the same strategy to skeletal mesh reconstruction.
						// FGeometryCollectionConversion::AppendSkeletalMesh(SourceSkeletalMesh, Source.SourceMaterial, Source.LocalTransform, GeometryCollectionObject, false);
					}
					else if (const UGeometryCollection* SourceGeometryCollection = Cast<UGeometryCollection>(SourceMesh))
					{
						FGeometryCollectionEngineConversion::AppendGeometryCollection(SourceGeometryCollection, Source.SourceMaterial, Source.LocalTransform, GeometryCollectionObject, false);
					}
				}

				constexpr bool bHasInternalMaterials = false;
				GeometryCollectionObject->InitializeMaterials(bHasInternalMaterials);

				// attempt to keep previously-set materials if requested
				if (bKeepPreviousMaterials)
				{
					int32 NewMatNum = GeometryCollectionObject->Materials.Num(), OldMatNum = OldMaterials.Num();
					GeometryCollectionObject->Materials.SetNum(OldMatNum);
					for (int32 MatIdx = 0; MatIdx < OldMatNum; MatIdx++)
					{
						GeometryCollectionObject->Materials[MatIdx] = OldMaterials[MatIdx];
					}
				}

				const UFractureModeSettings* ModeSettings = GetDefault<UFractureModeSettings>();
				ModeSettings->ApplyDefaultSettings(*GeometryCollection);

				FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
				AddSingleRootNodeIfRequired(GeometryCollectionObject);

				GeometryCollectionObject->RebuildRenderData();
			}

			GeometryCollectionObject->MarkPackageDirty();
		}
		
		GeometryCollectionComponent->InitializeEmbeddedGeometry();
		
		FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection(true);
		EditBoneColor.ResetBoneSelection();
		EditBoneColor.ResetHighlightedBones();
	}
	InToolkit.Pin()->OnSetLevelViewValue(-1);
	InToolkit.Pin()->SetOutlinerComponents(GeomCompSelection.Array());
}


#undef LOCTEXT_NAMESPACE

