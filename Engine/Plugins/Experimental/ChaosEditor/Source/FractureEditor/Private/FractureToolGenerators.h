// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolGenerators.generated.h"

class AGeometryCollectionActor;
class UGeometryCollection;
class UDataflow;

UCLASS()
class UFractureToolGenerateAsset : public UFractureActionTool
{
public:
	GENERATED_BODY()

	struct FCreationParameters
	{
		FString AssetPath;
		bool bFromToMeshTool = false;
		bool bSplitIslands = false;
		bool bUseRootProxies = true;
		TObjectPtr<UDataflow> DataflowAsset;
		TObjectPtr<UPhysicalMaterial> PhysicalMaterial;
	};

	UFractureToolGenerateAsset(const FObjectInitializer& ObjInit) 
		: Super(ObjInit)
		, AssetPath(FString(TEXT("/Game")))
	{}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual bool CanExecute() const override;

private:
	void OpenGenerateAssetDialog(TArray<AActor*>& Actors);
	void OnGenerateAssetPathChosen(const FCreationParameters& Params, TArray<AActor*> Actors);
	AGeometryCollectionActor* ConvertActorsToGeometryCollection(const FCreationParameters& Params, bool bAddInternalMaterials, TArray<AActor*>& Actors);
	class AGeometryCollectionActor* CreateNewGeometryActor(const FString& Name, const FTransform& Transform, bool AddMaterials /*= false*/);
	static AActor* AddActor(ULevel* InLevel, UClass* Class);

	FString GetDefaultAssetPath(const TArray<AActor*>& Actors) const;

private:
	TWeakPtr<FFractureEditorModeToolkit> Toolkit;

	UPROPERTY()
	FString AssetPath;

	UPROPERTY()
	TObjectPtr<UDataflow> LastDataflowAsset = nullptr;

	UPROPERTY()
	TObjectPtr<UPhysicalMaterial> LastPhysicalMaterial = nullptr;

	bool bLastSplitComponents = false;
	bool bLastFromToMesh = false;
	bool bLastUseRootProxies = true;
};



UCLASS(config = EditorPerProjectUserSettings)
class UGeometryCollectionResetSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UGeometryCollectionResetSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Whether to reset the materials along with the rest of the collection, or attempt to keep the current materials across the reset. */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResetMaterials = true;
};


UCLASS()
class UFractureToolResetAsset : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolResetAsset(const FObjectInitializer& ObjInit);

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual bool CanExecute() const override;

	// UFractureModalTool Interface
	virtual FText GetApplyText() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;

private:
	UPROPERTY()
	TObjectPtr<UGeometryCollectionResetSettings> ResetSettings;
};


