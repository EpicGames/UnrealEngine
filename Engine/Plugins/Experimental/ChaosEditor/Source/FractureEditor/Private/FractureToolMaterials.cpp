// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolMaterials.h"

#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"

#include "FractureEngineMaterials.h"
#include "ScopedTransaction.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolMaterials)

#define LOCTEXT_NAMESPACE "FractureToolMaterials"

void UFractureMaterialsSettings::AddMaterialSlot()
{
	UFractureToolMaterials* MaterialsTool = Cast<UFractureToolMaterials>(OwnerTool.Get());
	MaterialsTool->AddMaterialSlot();
}

void UFractureMaterialsSettings::RemoveMaterialSlot()
{
	UFractureToolMaterials* MaterialsTool = Cast<UFractureToolMaterials>(OwnerTool.Get());
	MaterialsTool->RemoveMaterialSlot();
}

void UFractureMaterialsSettings::UseAssetMaterialsOnComponents()
{
	UFractureToolMaterials* MaterialsTool = Cast<UFractureToolMaterials>(OwnerTool.Get());
	MaterialsTool->ClearMaterialOverridesOnComponents(bOnlySelectedComponents);
}

void UFractureToolMaterials::ClearMaterialOverridesOnComponents(bool bOnlySelectedComponents)
{
	if (!ActiveSelectedComponent.IsValid())
	{
		return;
	}

	// update materials on all components using this GeometryCollection
	if (!bOnlySelectedComponents)
	{
		for (FThreadSafeObjectIterator Iter(UGeometryCollectionComponent::StaticClass()); Iter; ++Iter)
		{
			UGeometryCollectionComponent* GCComponent = Cast<UGeometryCollectionComponent>(*Iter);
			if (GCComponent->GetRestCollection() == ActiveSelectedComponent->GetRestCollection())
			{
				GCComponent->EmptyOverrideMaterials();
				GCComponent->MarkRenderStateDirty();
			}
		}
	}
	else
	{
		TSet<UGeometryCollectionComponent*> GeomCompSelection;
		GetSelectedGeometryCollectionComponents(GeomCompSelection);
		for (UGeometryCollectionComponent* Component : GeomCompSelection)
		{
			if (Component->GetRestCollection() == ActiveSelectedComponent->GetRestCollection())
			{
				Component->EmptyOverrideMaterials();
				Component->MarkRenderStateDirty();
			}
		}
	}
}

void UFractureToolMaterials::RemoveMaterialSlot()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	FScopedTransaction Transaction(LOCTEXT("RemoveMaterialSlot", "Remove Material from Geometry Collection(s)"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FGeometryCollectionEdit Edit(GeometryCollectionComponent, GeometryCollection::EEditUpdate::Rest);
		UGeometryCollection* Collection = Edit.GetRestCollection();
		if (Collection->RemoveLastMaterialSlot())
		{
			Collection->RebuildRenderData();
		}
	}
	UpdateActiveMaterialsInfo();
}

void UFractureToolMaterials::AddMaterialSlot()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	FScopedTransaction Transaction(LOCTEXT("AddMaterialSlot", "Add Material to Geometry Collection(s)"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FGeometryCollectionEdit Edit(GeometryCollectionComponent, GeometryCollection::EEditUpdate::Rest);
		UGeometryCollection* Collection = Edit.GetRestCollection();

		int32 NewSlotIdx = Collection->AddNewMaterialSlot();

		Collection->RebuildRenderData();
	}
	UpdateActiveMaterialsInfo();
}

UFractureToolMaterials::UFractureToolMaterials(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	MaterialsSettings = NewObject<UFractureMaterialsSettings>(GetTransientPackage(), UFractureMaterialsSettings::StaticClass());
	MaterialsSettings->OwnerTool = this;
}

bool UFractureToolMaterials::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolMaterials::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolMaterials", "Edit geometry collection materials and default material assignments for new faces"));
}

FText UFractureToolMaterials::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolMaterialsTooltip", "Allows direct editing of materials on a geometry collection, as well as editing of the default handling."));
}

FSlateIcon UFractureToolMaterials::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Material");
}

void UFractureToolMaterials::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Material", "Material", "Update geometry materials, especially for new internal faces resulting from fracture.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->Materials = UICommandInfo;
}

TArray<UObject*> UFractureToolMaterials::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(MaterialsSettings);
	return Settings;
}

void UFractureToolMaterials::FractureContextChanged()
{
}

void UFractureToolMaterials::SelectedBonesChanged()
{
	UpdateActiveMaterialsInfo();
}

void UFractureToolMaterials::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
}

void UFractureToolMaterials::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(UFractureMaterialsSettings, Materials))
	{
		if (ActiveSelectedComponent.IsValid())
		{
			FGeometryCollectionEdit GCEdit = ActiveSelectedComponent->EditRestCollection(GeometryCollection::EEditUpdate::Rest);
			GCEdit.GetRestCollection()->Materials = MaterialsSettings->Materials;
			GCEdit.GetRestCollection()->RebuildRenderData();
			UpdateActiveMaterialsInfo();
		}
	}
}

TArray<FFractureToolContext> UFractureToolMaterials::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FFractureToolContext FullSelection(GeometryCollectionComponent);
		FullSelection.ConvertSelectionToRigidNodes();
		Contexts.Add(FullSelection);
		// TODO: consider also visualizing which faces will be updated
	}

	return Contexts;
}


void UFractureToolMaterials::UpdateActiveMaterialsInfo()
{
	// Choose a single asset to update for the asset materials options
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection, true);
	const UGeometryCollection* RestCollection = nullptr;
	UGeometryCollectionComponent* FoundComponent = nullptr;
	for (UGeometryCollectionComponent* Component : GeomCompSelection)
	{
		FoundComponent = Component;
		RestCollection = Component->GetRestCollection();
		if (RestCollection)
		{
			break;
		}
	}
	bool bHaveTargetCollectionUpdate = false;
	if (RestCollection)
	{
		ActiveSelectedComponent = FoundComponent;
		bHaveTargetCollectionUpdate = true;
		int32 NumMaterials = RestCollection->Materials.Num();
		MaterialsSettings->Materials = RestCollection->Materials;
		RestCollection->GetName(MaterialsSettings->EditingCollection);
	}
	else
	{
		bHaveTargetCollectionUpdate = false;
		MaterialsSettings->EditingCollection = LOCTEXT("NoActiveGeometryCollection", "None").ToString();
		MaterialsSettings->Materials.Empty();
		ActiveSelectedComponent = nullptr;
	}
	if (MaterialsSettings->bHaveTargetCollection != bHaveTargetCollectionUpdate)
	{
		MaterialsSettings->bHaveTargetCollection = bHaveTargetCollectionUpdate;
		NotifyOfPropertyChangeByTool(MaterialsSettings);
	}
	
	// Set active material list from full selection, since the material assignment applies to all selected collections
	MaterialsSettings->UpdateActiveMaterialNames(GetSelectedComponentMaterialNames(false, false));
}


int32 UFractureToolMaterials::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

		

		// Get ID to set and make sure it's valid
		int32 MatID = MaterialsSettings->GetAssignMaterialID();
		if (MatID == INDEX_NONE || MatID >= FractureContext.GetGeometryCollectionComponent()->GetNumMaterials())
		{
			return INDEX_NONE;
		}

		// convert enum to matching fracture engine materials enum
		FFractureEngineMaterials::ETargetFaces TargetFaces =
			(MaterialsSettings->ToFaces == EMaterialAssignmentTargets::AllFaces) ? FFractureEngineMaterials::ETargetFaces::AllFaces :
			(MaterialsSettings->ToFaces == EMaterialAssignmentTargets::OnlyInternalFaces) ? FFractureEngineMaterials::ETargetFaces::InternalFaces :
			FFractureEngineMaterials::ETargetFaces::ExternalFaces;

		if (MaterialsSettings->bOnlySelectedBones)
		{
			FFractureEngineMaterials::SetMaterial(Collection, FractureContext.GetSelection(), TargetFaces, MatID);
		}
		else
		{
			FFractureEngineMaterials::SetMaterialOnAllGeometry(Collection, TargetFaces, MatID);
		}

		Collection.ReindexMaterials();
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE

