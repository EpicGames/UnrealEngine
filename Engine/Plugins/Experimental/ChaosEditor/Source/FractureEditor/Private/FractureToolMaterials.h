// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolMaterials.generated.h"

class FFractureToolContext;
struct FMeshDescription;

UENUM()
enum class EMaterialAssignmentTargets
{
	OnlyInternalFaces,
	OnlyExternalFaces,
	AllFaces
};

/** Settings related to setting materials on a Geometry Collection **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureMaterialsSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureMaterialsSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}


	/** Geometry Collection selected for editing */
	UPROPERTY(VisibleAnywhere, Category = AssetMaterials, meta = (NoResetToDefault))
	FString EditingCollection;
	
	/** Materials on the selected Geometry Collection's underlying asset (the Rest Collection). */
	UPROPERTY(EditAnywhere, Category = AssetMaterials, EditFixedSize, meta = (DisplayName = "Asset Materials", NoResetToDefault, EditCondition = "bHaveTargetCollection", HideEditConditionToggle))
	TArray<TObjectPtr<UMaterialInterface>> Materials;
	
	/** Add a new material slot to the selected geometry collections */
	UFUNCTION(CallInEditor, Category = AssetMaterials)
	void AddMaterialSlot();

	/** Remove the last material slot from the selected Geometry Collections. (Will not remove the final material.) */
	UFUNCTION(CallInEditor, Category = AssetMaterials)
	void RemoveMaterialSlot();

	/** Clear material overrides on components using this asset, so the asset materials are used */
	UFUNCTION(CallInEditor, Category = ComponentMaterials)
	void UseAssetMaterialsOnComponents();

	/** Whether 'Use Asset Materials On Components' should only update the selected components, or update all components using this asset */
	UPROPERTY(EditAnywhere, Category = ComponentMaterials)
	bool bOnlySelectedComponents = false;

	/** Material to assign to selected faces */
	UPROPERTY(EditAnywhere, Category = AssignMaterials, meta = (TransientToolProperty, DisplayName = "Assign Material", GetOptions = GetMaterialNamesFunc, NoResetToDefault))
	FString AssignMaterial;

	/** Which subset of faces to update materials assignments on, for the selected geometry */
	UPROPERTY(EditAnywhere, Category = AssignMaterials)
	EMaterialAssignmentTargets ToFaces = EMaterialAssignmentTargets::OnlyInternalFaces;

	/** Whether to only assign materials for faces in the selected bones, or the whole geometry collection */
	UPROPERTY(EditAnywhere, Category = AssignMaterials)
	bool bOnlySelectedBones = true;

	UFUNCTION()
	const TArray<FString>& GetMaterialNamesFunc() { return AssignMaterialNamesList; }

	void UpdateActiveMaterialNames(TArray<FString> InMaterialNamesList)
	{
		AssignMaterialNamesList = InMaterialNamesList;
		if (!AssignMaterialNamesList.Contains(AssignMaterial))
		{
			AssignMaterial = AssignMaterialNamesList.IsEmpty() ? FString() : AssignMaterialNamesList[0];
		}
	}

	int32 GetAssignMaterialID()
	{
		return AssignMaterialNamesList.Find(AssignMaterial);
	}

	UPROPERTY()
	bool bHaveTargetCollection = false;

private:

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> AssignMaterialNamesList;

};


UCLASS(DisplayName = "Materials Tool", Category = "FractureTools")
class UFractureToolMaterials : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolMaterials(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureToolMaterials", "ExecuteMaterials", "Assign Materials")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;
	virtual bool ExecuteUpdatesShape() const override
	{
		return false;
	}
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	virtual void SelectedBonesChanged() override;

	// Called when the modal tool is entered
	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override
	{
		Super::Setup(InToolkit);
		UpdateActiveMaterialsInfo();
	}

	virtual void Shutdown()
	{
		Super::Shutdown();
		ActiveSelectedComponent = nullptr;
	}

	void UpdateActiveMaterialsInfo();

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	void RemoveMaterialSlot();
	void AddMaterialSlot();
	void ClearMaterialOverridesOnComponents(bool bOnlySelected = false);

protected:
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureMaterialsSettings> MaterialsSettings;

	// Track the geometry collection asset last used to populate the materials list, if any
	UPROPERTY()
	TWeakObjectPtr<UGeometryCollectionComponent> ActiveSelectedComponent = nullptr;
};
