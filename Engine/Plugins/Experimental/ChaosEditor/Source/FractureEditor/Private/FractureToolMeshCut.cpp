// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolMeshCut.h"
#include "FractureEditorStyle.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "Algo/RemoveIf.h"

#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "FractureToolBackgroundTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolMeshCut)

using namespace UE::Fracture;

#define LOCTEXT_NAMESPACE "FractureMesh"


UFractureToolMeshCut::UFractureToolMeshCut(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	MeshCutSettings = NewObject<UFractureMeshCutSettings>(GetTransientPackage(), UFractureMeshCutSettings::StaticClass());
	MeshCutSettings->OwnerTool = this;
}

FText UFractureToolMeshCut::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolMeshCut", "Mesh Cut Fracture")); 
}

FText UFractureToolMeshCut::GetTooltipText() const 
{
	return FText(NSLOCTEXT("Fracture", "FractureToolMeshCutTooltip", "Mesh fracture can be used to make cuts along a mesh in your Geometry Collection. Click the Fracture Button to commit the fracture to the geometry collection."));
}

FSlateIcon UFractureToolMeshCut::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Mesh");
}

void UFractureToolMeshCut::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Mesh", "Mesh", "Fracture using the shape of a chosen static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Mesh = UICommandInfo;
}

TArray<UObject*> UFractureToolMeshCut::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(MeshCutSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

bool UFractureToolMeshCut::IsCuttingActorValid()
{
	const UFractureMeshCutSettings* LocalCutSettings = MeshCutSettings;
	AStaticMeshActor* CuttingActor = LocalCutSettings->CuttingActor.Get();
	if (CuttingActor == nullptr)
	{
		return false;
	}
	const UStaticMeshComponent* Component = CuttingActor->GetStaticMeshComponent();
	if (Component == nullptr)
	{
		return false;
	}
	const UStaticMesh* Mesh = Component->GetStaticMesh();
	if (Mesh == nullptr)
	{
		return false;
	}
	if (Mesh->GetNumLODs() < 1)
	{
		return false;
	}
	return true;
}

void UFractureToolMeshCut::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (CutterSettings->bDrawDiagram && IsCuttingActorValid())
	{
		FBox Box = MeshCutSettings->CuttingActor.Get()->GetStaticMeshComponent()->GetStaticMesh()->GetBoundingBox();
		
		EnumerateVisualizationMapping(TransformsMappings, RenderMeshTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			const FTransform& Transform = RenderMeshTransforms[Idx];
			FVector B000 = ExplodedVector + Transform.TransformPosition(Box.Min);
			FVector B111 = ExplodedVector + Transform.TransformPosition(Box.Max);
			FVector B011 = ExplodedVector + Transform.TransformPosition(FVector(Box.Min.X, Box.Max.Y, Box.Max.Z));
			FVector B101 = ExplodedVector + Transform.TransformPosition(FVector(Box.Max.X, Box.Min.Y, Box.Max.Z));
			FVector B110 = ExplodedVector + Transform.TransformPosition(FVector(Box.Max.X, Box.Max.Y, Box.Min.Z));
			FVector B001 = ExplodedVector + Transform.TransformPosition(FVector(Box.Min.X, Box.Min.Y, Box.Max.Z));
			FVector B010 = ExplodedVector + Transform.TransformPosition(FVector(Box.Min.X, Box.Max.Y, Box.Min.Z));
			FVector B100 = ExplodedVector + Transform.TransformPosition(FVector(Box.Max.X, Box.Min.Y, Box.Min.Z));

			PDI->DrawLine(B000, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B000, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B000, B001, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B111, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B111, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B111, B110, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B001, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B001, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B110, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B110, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B100, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
			PDI->DrawLine(B010, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		});
	}
}

void UFractureToolMeshCut::GenerateMeshTransforms(const FFractureToolContext& Context, TArray<FTransform>& MeshTransforms)
{
	FRandomStream RandStream(Context.GetSeed());

	FBox Bounds = Context.GetWorldBounds();
	const FVector Extent(Bounds.Max - Bounds.Min);

	TArray<FVector> Positions;
	if (MeshCutSettings->CutDistribution == EMeshCutDistribution::UniformRandom)
	{
		Positions.Reserve(MeshCutSettings->NumberToScatter);
		for (int32 Idx = 0; Idx < MeshCutSettings->NumberToScatter; ++Idx)
		{
			Positions.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		}
	}
	else if (MeshCutSettings->CutDistribution == EMeshCutDistribution::Grid)
	{
		Positions.Reserve(MeshCutSettings->GridX * MeshCutSettings->GridY * MeshCutSettings->GridZ);
		auto ToFrac = [](int32 Val, int32 NumVals) -> FVector::FReal
		{
			return (FVector::FReal(Val) + FVector::FReal(.5)) / FVector::FReal(NumVals);
		};
		for (int32 X = 0; X < MeshCutSettings->GridX; ++X)
		{
			FVector::FReal XFrac = ToFrac(X, MeshCutSettings->GridX);
			for (int32 Y = 0; Y < MeshCutSettings->GridY; ++Y)
			{
				FVector::FReal YFrac = ToFrac(Y, MeshCutSettings->GridY);
				for (int32 Z = 0; Z < MeshCutSettings->GridZ; ++Z)
				{
					FVector::FReal ZFrac = ToFrac(Z, MeshCutSettings->GridZ);
					Positions.Emplace(Bounds.Min + FVector(XFrac, YFrac, ZFrac) * Extent);
				}
			}
		}

		for (FVector& Position : Positions)
		{
			Position += (RandStream.VRand() * RandStream.FRand() * MeshCutSettings->Variability);
		}
	}

	MeshTransforms.Reserve(MeshTransforms.Num() + Positions.Num());
	for (const FVector& Position : Positions)
	{
		FVector::FReal Scale = RandStream.FRandRange(MeshCutSettings->MinScaleFactor, MeshCutSettings->MaxScaleFactor);
		FVector ScaleVec(Scale, Scale, Scale);
		FRotator Orientation = FRotator::ZeroRotator;
		if (MeshCutSettings->bRandomOrientation)
		{
			Orientation = FRotator(
				RandStream.FRandRange(-MeshCutSettings->PitchRange, MeshCutSettings->PitchRange),
				RandStream.FRandRange(-MeshCutSettings->YawRange, MeshCutSettings->YawRange),
				RandStream.FRandRange(-MeshCutSettings->RollRange, MeshCutSettings->RollRange)
			);
		}
		MeshTransforms.Emplace(FTransform(Orientation, Position, ScaleVec));
	}
}

void UFractureToolMeshCut::FractureContextChanged()
{
	SetMandateGroupFracture(MeshCutSettings->CutDistribution == EMeshCutDistribution::SingleCut);

	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		FBox Bounds = FractureContext.GetWorldBounds();
		if (!Bounds.IsValid)
		{
			continue;
		}
		int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
		int32 BoneIdx = FractureContext.GetSelection().Num() == 1 ? FractureContext.GetSelection()[0] : INDEX_NONE;
		TransformsMappings.AddMapping(CollectionIdx, BoneIdx, RenderMeshTransforms.Num());

		GenerateMeshTransforms(FractureContext, RenderMeshTransforms);
	}
}

class FMeshFractureOp : public FGeometryCollectionFractureOperator
{
public:
	FMeshFractureOp(const FGeometryCollection& SourceCollection) : FGeometryCollectionFractureOperator(SourceCollection)
	{}

	virtual ~FMeshFractureOp() = default;

	TArray<int> Selection;
	TArray<FTransform> MeshTransforms;
	float PointSpacing;
	int Seed;
	FTransform Transform;
	UE::Geometry::FDynamicMesh3 CuttingMesh;
	bool bSplitIslands = true;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		// Note: Noise not currently supported
		FInternalSurfaceMaterials InternalSurfaceMaterials;

		ResultGeometryIndex = -1;
		const float ProgressFrac = 1.0 / MeshTransforms.Num();

		int32 OriginalNumTransforms = CollectionCopy->NumElements(FGeometryCollection::TransformGroup);
		
		TArray<int32> BonesToCut = Selection;
		for (const FTransform& ScatterTransform : MeshTransforms)
		{
			if (Progress && Progress->Cancelled())
			{
				return;
			}

			FProgressCancel::FProgressScope ProgressScope(Progress, ProgressFrac);
			constexpr bool bSetDefaultInternalMaterialsFromCollection = true;
			int32 Index = CutWithMesh(CuttingMesh, ScatterTransform, InternalSurfaceMaterials, *CollectionCopy, BonesToCut, PointSpacing, Transform, bSetDefaultInternalMaterialsFromCollection, Progress, FIslandSplitSettings(bSplitIslands, CloseVertexDistance, VertexToSurfaceBridgeDistance));
			if (Progress && Progress->Cancelled())
			{
				return;
			}

			int32 NewLen = Algo::RemoveIf(BonesToCut, [&](int32 Bone)
				{
					return !CollectionCopy->IsVisible(Bone); // remove already-fractured pieces from the to-cut list
				});
			BonesToCut.SetNum(NewLen);
			if (ResultGeometryIndex == -1)
			{
				ResultGeometryIndex = Index;
			}
			if (Index > -1)
			{
				int32 TransformIdx = CollectionCopy->TransformIndex[Index];
				// after a successful cut, also consider any new bones added by the cut
				for (int32 NewBoneIdx = TransformIdx; NewBoneIdx < CollectionCopy->NumElements(FGeometryCollection::TransformGroup); NewBoneIdx++)
				{
					BonesToCut.Add(NewBoneIdx);
				}
			}
		}

		if (ResultGeometryIndex > -1)
		{
			TArray<int32> ToRemove;
			for (int32 NewIdx = OriginalNumTransforms; NewIdx < CollectionCopy->NumElements(FGeometryCollection::TransformGroup); ++NewIdx)
			{
				if (CollectionCopy->IsRigid(NewIdx))
				{
					int32 ParentIdx = CollectionCopy->Parent[NewIdx];
					if (ParentIdx >= OriginalNumTransforms)
					{
						do
						{
							ParentIdx = CollectionCopy->Parent[ParentIdx];
						} while (CollectionCopy->Parent[ParentIdx] >= OriginalNumTransforms);
						CollectionCopy->ParentTransforms(ParentIdx, NewIdx);
					}
				}
				else
				{
					ToRemove.Add(NewIdx);
				}
			}
			FManagedArrayCollection::FProcessingParameters ProcessingParams;
			ProcessingParams.bDoValidation = false;
			CollectionCopy->RemoveElements(FGeometryCollection::TransformGroup, ToRemove, ProcessingParams);
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}
		SetResult(MoveTemp(CollectionCopy));
	}
};

int32 UFractureToolMeshCut::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		const UFractureMeshCutSettings* LocalCutSettings = MeshCutSettings;

		if (!IsCuttingActorValid())
		{
			return INDEX_NONE;
		}

		TUniquePtr<FMeshFractureOp> MeshCutOp = MakeUnique<FMeshFractureOp>(*(FractureContext.GetGeometryCollection()));
		MeshCutOp->Selection = FractureContext.GetSelection();
		MeshCutOp->PointSpacing = CollisionSettings->GetPointSpacing();
		MeshCutOp->Seed = FractureContext.GetSeed();
		MeshCutOp->Transform = FractureContext.GetTransform();
		MeshCutOp->bSplitIslands = CutterSettings->bSplitIslands;
		MeshCutOp->CloseVertexDistance = CutterSettings->CloseVertexDistance;
		MeshCutOp->VertexToSurfaceBridgeDistance = CutterSettings->VertexToSurfaceBridgeDistance;
		if (LocalCutSettings->CutDistribution == EMeshCutDistribution::SingleCut)
		{
			MeshCutOp->MeshTransforms.Add(LocalCutSettings->CuttingActor.Get()->GetTransform());
		}
		else
		{
			GenerateMeshTransforms(FractureContext, MeshCutOp->MeshTransforms);
		}
		FMeshDescription* MeshDescription = LocalCutSettings->CuttingActor.Get()->GetStaticMeshComponent()->GetStaticMesh()->GetMeshDescription(0);
		MeshCutOp->CuttingMesh = ConvertMeshDescriptionToCuttingDynamicMesh(MeshDescription, FractureContext.GetGeometryCollection()->NumUVLayers());

		int Result = RunCancellableGeometryCollectionOp<FMeshFractureOp>(*(FractureContext.GetGeometryCollection()),
			MoveTemp(MeshCutOp), LOCTEXT("ComputingMeshFractureMessage", "Computing Mesh Fracture"));
		return Result;
	}

	return INDEX_NONE;
}


#undef LOCTEXT_NAMESPACE
