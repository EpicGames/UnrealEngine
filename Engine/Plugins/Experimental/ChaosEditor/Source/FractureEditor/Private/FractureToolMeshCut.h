// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "Engine/StaticMeshActor.h"

#include "FractureToolMeshCut.generated.h"

class FFractureToolContext;

UENUM()
enum class EMeshCutDistribution
{
	// Cut only once, at the cutting mesh's current location in the level
	SingleCut,
	// Scatter the cutting mesh in a uniform random distribution around the geometry bounding box
	UniformRandom,
	// Arrange the cutting mesh in a regular grid pattern
	Grid
};

UCLASS(config = EditorPerProjectUserSettings)
class UFractureMeshCutSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureMeshCutSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit) {}

	/** Static Mesh Actor to be used as a cutting surface. For best results, use a closed, watertight mesh */
	UPROPERTY(EditAnywhere, Category = MeshCut, meta = (DisplayName = "Cutting Actor"))
	TSoftObjectPtr<AStaticMeshActor> CuttingActor;

	/** How to arrange the mesh cuts in space */
	UPROPERTY(EditAnywhere, Category = Distribution)
	EMeshCutDistribution CutDistribution = EMeshCutDistribution::SingleCut;

	/** Number of meshes to random scatter */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (ClampMin = "1", UIMax = "5000",
		EditCondition = "CutDistribution == EMeshCutDistribution::UniformRandom", EditConditionHides))
	int NumberToScatter = 10;

	/** Number of meshes to add to grid in X */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "Grid Width", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "CutDistribution == EMeshCutDistribution::Grid", EditConditionHides))
	int32 GridX = 5;

	/** Number of meshes to add to grid in Y */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "Grid Depth", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "CutDistribution == EMeshCutDistribution::Grid", EditConditionHides))
	int32 GridY = 5;

	/** Number of meshes to add to grid in Z */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "Grid Height", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "CutDistribution == EMeshCutDistribution::Grid", EditConditionHides))
	int32 GridZ = 5;

	/** Magnitude of random displacement to cutting meshes */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "Variability", EditCondition = "CutDistribution == EMeshCutDistribution::Grid", EditConditionHides, UIMin = "0.0", ClampMin = "0.0"))
	float Variability = 0.0f;

	/** Minimum scale factor to apply to cutting meshes. A random scale will be chosen between Min and Max */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (ClampMin = "0.001", EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut", EditConditionHides))
	float MinScaleFactor = .5;

	/** Maximum scale factor to apply to cutting meshes. A random scale will be chosen between Min and Max */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (ClampMin = "0.001", EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut", EditConditionHides))
	float MaxScaleFactor = 1.5;

	/** Whether to randomly vary the orientation of the cutting meshes */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut", EditConditionHides))
	bool bRandomOrientation = true;

	/** Roll will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "+/- Roll Range", EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float RollRange = 180;

	/** Pitch will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "+/- Pitch Range", EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float PitchRange = 180;

	/** Yaw will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DisplayName = "+/- Yaw Range", EditCondition = "CutDistribution != EMeshCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float YawRange = 180;
};


UCLASS(DisplayName = "Mesh Cut Tool", Category = "FractureTools")
class UFractureToolMeshCut : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolMeshCut(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;

	virtual void ConfigureCutterSettings() override
	{
		CutterSettings->bGroupFractureToggleEnabled = true;
		CutterSettings->bDrawSitesToggleEnabled = false;
		CutterSettings->bNoisePreviewToggleEnabled = false;
		CutterSettings->bNoisePreviewHasScale = true;
		CutterSettings->bGroutSettingEnabled = false;
		CutterSettings->bNoiseSettingsEnabled = false;
	}

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		RenderMeshTransforms.Empty();
		TransformsMappings.Empty();
	}

private:
	// Slicing
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureMeshCutSettings> MeshCutSettings;

	// check if the chosen actor can be used to cut the geometry collection (i.e. if it is a valid static mesh actor with a non-empty static mesh)
	bool IsCuttingActorValid();

	void GenerateMeshTransforms(const FFractureToolContext& Context, TArray<FTransform>& MeshTransforms);

	TArray<FTransform> RenderMeshTransforms;
	FVisualizationMappings TransformsMappings;
};


