// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolPlaneCut.h"
#include "FractureEditorStyle.h"
#include "FractureSettings.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"
#include "Drawing/MeshDebugDrawing.h"
#include "FrameTypes.h"
#include "FractureToolBackgroundTask.h"
#include "MeshOpPreviewHelpers.h"
#include "Materials/MaterialInterface.h"
#include "ToolSetupUtil.h"
#include "Generators/RectangleMeshGenerator.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolPlaneCut)

using namespace UE::Fracture;

#define LOCTEXT_NAMESPACE "FracturePlanar"

// This op runs in a background thread to generate a preview of the cutting mesh, with noise applied
class FPlaneNoisePreviewOp : public UE::Geometry::FDynamicMeshOperator
{
public:
	virtual ~FPlaneNoisePreviewOp() {}

	// inputs
	TArray<FTransform> PlaneTransforms;
	TArray<FVector> NoisePivots, ExplodedVectors;
	TArray<FNoiseOffsets> NoiseOffsets;
	FNoiseSettings NoiseSettings;
	float PlaneSize;

	// FDynamicMeshOperator implementation
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if (PlaneTransforms.IsEmpty())
		{
			return;
		}

		UE::Geometry::FDynamicMeshEditor MeshEditor(ResultMesh.Get());
		FVector Center(0, 0, 0);
		for (const FTransform& Transform : PlaneTransforms)
		{
			Center += Transform.GetLocation();
		}
		Center /= (double)PlaneTransforms.Num();
		UE::Geometry::FTransformSRT3d CenteringTransform(Center);
		SetResultTransform(CenteringTransform);

		for (int32 PlaneIdx = 0; PlaneIdx < PlaneTransforms.Num(); ++PlaneIdx)
		{
			UE::Geometry::FRectangleMeshGenerator RectangleGenerator;
			RectangleGenerator.Width = PlaneSize;
			RectangleGenerator.Height = PlaneSize;
			
			constexpr int32 MaxCountPerDim = 2000;
			RectangleGenerator.WidthVertexCount = FMath::Min(MaxCountPerDim, RectangleGenerator.Width / NoiseSettings.PointSpacing);
			RectangleGenerator.HeightVertexCount = FMath::Min(MaxCountPerDim, RectangleGenerator.Height / NoiseSettings.PointSpacing);

			if (Progress && Progress->Cancelled())
			{
				return;
			}

			FDynamicMesh3 RectMesh(&RectangleGenerator.Generate());
			const FTransform& PlaneTransform = PlaneTransforms[PlaneIdx];
			const FVector& NoisePivot = NoisePivots[PlaneIdx];
			const FNoiseOffsets& NoiseOffset = NoiseOffsets[PlaneIdx];
			const FVector& ExplodedVector = ExplodedVectors[PlaneIdx];
			FVector Normal = PlaneTransform.GetUnitAxis(EAxis::Z);
			for (int VID : RectMesh.VertexIndicesItr())
			{
				FVector3d WorldPos = PlaneTransform.TransformPosition(RectMesh.GetVertex(VID));
				FVector3d NewWorldPos = WorldPos + NoiseSettings.NoiseVector(WorldPos - NoisePivot, NoiseOffset).Dot(Normal) * Normal;
				RectMesh.SetVertex(VID, NewWorldPos + ExplodedVector - Center);
			}
			UE::Geometry::FMeshIndexMappings Mappings;
			MeshEditor.AppendMesh(&RectMesh, Mappings);
		}
		UE::Geometry::FMeshNormals::QuickRecomputeOverlayNormals(*ResultMesh);
	}
};


TUniquePtr<UE::Geometry::FDynamicMeshOperator> UFractureToolPlaneCut::MakeNewOperator()
{
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	NoisePreviewExplodeAmount = FractureSettings->ExplodeAmount;

	TUniquePtr<FPlaneNoisePreviewOp> NoisePreviewOp = MakeUnique<FPlaneNoisePreviewOp>();
	CutterSettings->TransferNoiseSettings(NoisePreviewOp->NoiseSettings);

	if (GizmoSettings->IsGizmoEnabled()) // set preview from gizmo
	{
		NoisePreviewOp->PlaneSize = GizmoPlaneSize;
		const FTransform& Transform = GizmoSettings->GetTransform();
		EnumerateVisualizationMapping(PlanesMappings, RenderCuttingPlanesTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
			{
				NoisePreviewOp->PlaneTransforms.Add(Transform);
				NoisePreviewOp->ExplodedVectors.Add(ExplodedVector);
				NoisePreviewOp->NoiseOffsets.Add(NoiseOffsets[Idx]);
				NoisePreviewOp->NoisePivots.Add(NoisePivots[Idx]);
			});
	}
	else // set from computed transforms
	{
		NoisePreviewOp->PlaneSize = RenderCuttingPlaneSize;
		EnumerateVisualizationMapping(PlanesMappings, RenderCuttingPlanesTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
			{
				const FTransform& Transform = RenderCuttingPlanesTransforms[Idx];
				NoisePreviewOp->PlaneTransforms.Add(Transform);
				NoisePreviewOp->ExplodedVectors.Add(ExplodedVector);
				NoisePreviewOp->NoiseOffsets.Add(NoiseOffsets[Idx]);
				NoisePreviewOp->NoisePivots.Add(NoisePivots[Idx]);
			});
	}
	NoisePreviewOp->PlaneSize *= CutterSettings->NoisePreviewScale;

	return NoisePreviewOp;
}


UFractureToolPlaneCut::UFractureToolPlaneCut(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	PlaneCutSettings = NewObject<UFracturePlaneCutSettings>(GetTransientPackage(), UFracturePlaneCutSettings::StaticClass());
	PlaneCutSettings->OwnerTool = this;

	GizmoSettings = NewObject<UFractureTransformGizmoSettings>(GetTransientPackage(), UFractureTransformGizmoSettings::StaticClass());
	GizmoSettings->OwnerTool = this;
}


void UFractureToolPlaneCut::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	GizmoSettings->Setup(this);
	PlaneCutSettings->bCanCutWithMultiplePlanes = !GizmoSettings->bUseGizmo;
	NotifyOfPropertyChangeByTool(PlaneCutSettings);

	// Initialize the background compute object for the noise preview
	if (GEditor && !NoisePreview)
	{
		UWorld* World = GEditor->GetEditorWorldContext().World();
		NoisePreview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
		NoisePreview->Setup(World, this);

		TArray<UMaterialInterface*> Materials;
		// Note the ToolSetupUtil functions optionally take a UInteractiveToolManager to provide fallback materials,
		// but fracture mode does not set that up, so they may return a null material.
		// Note that if the working material is null, the preview will just always use the default material instead.
		UMaterialInterface* NoiseMaterial = ToolSetupUtil::GetDefaultSculptMaterial(nullptr);
		if (!NoiseMaterial)
		{
			NoiseMaterial = ToolSetupUtil::GetDefaultMaterial();
		}
		UMaterialInterface* WorkingMaterial = ToolSetupUtil::GetDefaultWorkingMaterial(nullptr);
		Materials.Add(NoiseMaterial);
		NoisePreview->ConfigureMaterials(Materials, WorkingMaterial);
	}
}


void UFractureToolPlaneCut::Shutdown()
{
	Super::Shutdown();
	GizmoSettings->Shutdown();

	if (NoisePreview)
	{
		NoisePreview->Shutdown();
		NoisePreview = nullptr;
	}
}


FText UFractureToolPlaneCut::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolPlaneCut", "Plane Cut Fracture")); 
}

FText UFractureToolPlaneCut::GetTooltipText() const 
{
	return FText(NSLOCTEXT("Fracture", "FractureToolPlaneCutTooltip", "Planar fracture can be used to make cuts along a plane in your Geometry Collection. You can apply noise to planar cuts for more organic results.  Click the Fracture Button to commit the fracture to the geometry collection."));
}

FSlateIcon UFractureToolPlaneCut::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Planar");
}

void UFractureToolPlaneCut::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Planar", "Planar", "Fracture with planes.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Planar = UICommandInfo;
}

void UFractureToolPlaneCut::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	// Note: ExplodeAmount changes generally do not trigger tool context changes, but can affect visualization
	// So we detect such changes here and invalidate the preview
	// We could instead try to translate the existing NoisePreview, but that would require a separate NoisePreview per plane
	UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();
	if (NoisePreview && NoisePreviewExplodeAmount != FractureSettings->ExplodeAmount)
	{
		NoisePreview->InvalidateResult();
	}

	const UFracturePlaneCutSettings* LocalCutSettings = PlaneCutSettings;
	if (CutterSettings->bDrawDiagram)
	{
		// Draw a point centered at plane origin, and a square on the plane around it.
		auto DrawPlane = [&PDI](const FTransform& Transform, float PlaneSize, FVector Offset)
		{
			FVector Center = Transform.GetLocation() + Offset;
			FVector X = (PlaneSize * .5) * Transform.GetUnitAxis(EAxis::X);
			FVector Y = (PlaneSize * .5) * Transform.GetUnitAxis(EAxis::Y);
			FVector Corners[4]
			{
				Center - X - Y,
				Center + X - Y,
				Center + X + Y,
				Center - X + Y
			};
			PDI->DrawPoint(Center, FLinearColor::Green, 4.f, SDPG_Foreground);
			PDI->DrawLine(Corners[0], Corners[1], FLinearColor(255, 0, 0), SDPG_Foreground);
			PDI->DrawLine(Corners[1], Corners[2], FLinearColor(0, 255, 0), SDPG_Foreground);
			PDI->DrawLine(Corners[2], Corners[3], FLinearColor(255, 0, 0), SDPG_Foreground);
			PDI->DrawLine(Corners[3], Corners[0], FLinearColor(0, 255, 0), SDPG_Foreground);

			// Put some grid lines in that square
			const int32 NumGridLines = 20;
			const float GridLineSpacing = PlaneSize / (NumGridLines - 1);
			const FColor GridColor(64, 64, 64, 128);
			const float GridThickness = 1.0f;
			const UE::Geometry::FFrame3d DrawFrame(Center, Transform.GetRotation());
			MeshDebugDraw::DrawSimpleGrid(DrawFrame, NumGridLines, GridLineSpacing, GridThickness, GridColor, true, PDI, FTransform::Identity);
		};

		if (GizmoSettings->IsGizmoEnabled())
		{
			const FTransform& Transform = GizmoSettings->GetTransform();
			EnumerateVisualizationMapping(PlanesMappings, RenderCuttingPlanesTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
			{
				DrawPlane(Transform, GizmoPlaneSize, ExplodedVector);
			});
		}
		else // draw from computed transforms
		{
			EnumerateVisualizationMapping(PlanesMappings, RenderCuttingPlanesTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
			{
				const FTransform& Transform = RenderCuttingPlanesTransforms[Idx];
				DrawPlane(Transform, RenderCuttingPlaneSize, ExplodedVector);
			});
		}
	}
}

TArray<UObject*> UFractureToolPlaneCut::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(PlaneCutSettings);
	Settings.Add(GizmoSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}


void UFractureToolPlaneCut::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFractureTransformGizmoSettings, bUseGizmo))
	{
		PlaneCutSettings->bCanCutWithMultiplePlanes = !GizmoSettings->bUseGizmo;
		NotifyOfPropertyChangeByTool(PlaneCutSettings);
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}


void UFractureToolPlaneCut::ClearVisualizations()
{
	Super::ClearVisualizations();
	RenderCuttingPlanesTransforms.Empty();
	NoiseOffsets.Empty();
	NoisePivots.Empty();
	PlanesMappings.Empty();
	if (NoisePreview)
	{
		NoisePreview->InvalidateResult();
	}
}


void UFractureToolPlaneCut::OnTick(float DeltaTime)
{
	if (NoisePreview)
	{
		NoisePreview->Tick(DeltaTime);
	}

	Super::OnTick(DeltaTime);
}


void UFractureToolPlaneCut::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	if (NoisePreview)
	{
		NoisePreview->SetVisibility(CutterSettings->bDrawNoisePreview);
	}

	RenderCuttingPlaneSize = FLT_MAX;
	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		FBox Bounds = FractureContext.GetWorldBounds();
		if (!Bounds.IsValid)
		{
			continue;
		}
		int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
		int32 BoneIdx = FractureContext.GetSelection().Num() == 1 ? FractureContext.GetSelection()[0] : INDEX_NONE;
		int32 TransformsStart = RenderCuttingPlanesTransforms.Num();
		PlanesMappings.AddMapping(CollectionIdx, BoneIdx, TransformsStart);

		GenerateSliceTransforms(FractureContext, RenderCuttingPlanesTransforms);
		for (int32 Idx = TransformsStart; Idx < RenderCuttingPlanesTransforms.Num(); ++Idx)
		{
			FRandomStream RandomStream(FractureContext.GetSeed());
			FNoiseOffsets NoiseOffset(RandomStream);
			NoiseOffsets.Add(NoiseOffset);
			NoisePivots.Add(FractureContext.GetTransform().GetLocation());
		}

		if (Bounds.GetExtent().GetMax() < RenderCuttingPlaneSize)
		{
			RenderCuttingPlaneSize = Bounds.GetExtent().GetMax();
		}
	}

	if (NoisePreview)
	{
		NoisePreview->InvalidateResult();
	}
}


class FPlaneFractureOp : public FGeometryCollectionFractureOperator
{
public:
	FPlaneFractureOp(const FGeometryCollection& SourceCollection) : FGeometryCollectionFractureOperator(SourceCollection)
	{}

	virtual ~FPlaneFractureOp() = default;

	TArray<int> Selection;
	TArray<FPlane> CuttingPlanes;
	FInternalSurfaceMaterials InternalSurfaceMaterials;
	float PointSpacing;
	float Grout;
	int Seed;
	FTransform Transform;
	UE::Geometry::FDynamicMesh3 CuttingMesh;
	bool bSplitIslands = true;
	double CloseVertexDistance = 1e-3;
	double VertexToSurfaceBridgeDistance = 0;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		ResultGeometryIndex = CutMultipleWithMultiplePlanes(CuttingPlanes, InternalSurfaceMaterials, *CollectionCopy, Selection, Grout, PointSpacing, Seed, Transform, true, Progress, FIslandSplitSettings(bSplitIslands, CloseVertexDistance, VertexToSurfaceBridgeDistance));

		if (Progress && Progress->Cancelled())
		{
			return;
		}
		SetResult(MoveTemp(CollectionCopy));
	}
};


int32 UFractureToolPlaneCut::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		TUniquePtr<FPlaneFractureOp> PlaneCutOp = MakeUnique<FPlaneFractureOp>(*(FractureContext.GetGeometryCollection()));
		PlaneCutOp->Selection = FractureContext.GetSelection();
		PlaneCutOp->Grout = CutterSettings->Grout;
		PlaneCutOp->PointSpacing = CollisionSettings->GetPointSpacing();
		PlaneCutOp->Seed = FractureContext.GetSeed();
		PlaneCutOp->Transform = FractureContext.GetTransform();
		PlaneCutOp->bSplitIslands = CutterSettings->bSplitIslands;
		PlaneCutOp->CloseVertexDistance = CutterSettings->CloseVertexDistance;
		PlaneCutOp->VertexToSurfaceBridgeDistance = CutterSettings->VertexToSurfaceBridgeDistance;

		if (GizmoSettings->IsGizmoEnabled())
		{
			FTransform Transform = GizmoSettings->GetTransform();
			PlaneCutOp->CuttingPlanes.Add(FPlane(Transform.GetLocation(), Transform.GetUnitAxis(EAxis::Z)));
		}
		else
		{
			TArray<FTransform> CuttingPlaneTransforms;
			GenerateSliceTransforms(FractureContext, CuttingPlaneTransforms);
			for (const FTransform& Transform : CuttingPlaneTransforms)
			{
				PlaneCutOp->CuttingPlanes.Add(FPlane(Transform.GetLocation(), Transform.GetUnitAxis(EAxis::Z)));
			}
		}
		
		FNoiseSettings NoiseSettings;
		if (CutterSettings->Amplitude > 0.0f)
		{
			CutterSettings->TransferNoiseSettings(NoiseSettings);
			PlaneCutOp->InternalSurfaceMaterials.NoiseSettings = NoiseSettings;
		}

		int Result = RunCancellableGeometryCollectionOp<FPlaneFractureOp>(*(FractureContext.GetGeometryCollection()),
			MoveTemp(PlaneCutOp), LOCTEXT("ComputingPlaneFractureMessage", "Computing Plane Fracture"));
		return Result;
	}

	return INDEX_NONE;
}

void UFractureToolPlaneCut::GenerateSliceTransforms(const FFractureToolContext& Context, TArray<FTransform>& CuttingPlaneTransforms)
{
	FRandomStream RandStream(Context.GetSeed());

	FBox Bounds = Context.GetWorldBounds();
	const FVector Extent(Bounds.Max - Bounds.Min);

	CuttingPlaneTransforms.Reserve(CuttingPlaneTransforms.Num() + PlaneCutSettings->NumberPlanarCuts);
	for (int32 ii = 0; ii < PlaneCutSettings->NumberPlanarCuts; ++ii)
	{
		FVector Position(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
		CuttingPlaneTransforms.Emplace(FTransform(FRotator(RandStream.FRand() * 360.0f, RandStream.FRand() * 360.0f, 0.0f), Position));
	}
}

void UFractureToolPlaneCut::SelectedBonesChanged()
{
	GizmoSettings->ResetGizmo();
	Super::SelectedBonesChanged();
}

#undef LOCTEXT_NAMESPACE
