// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "ModelingOperators.h"

#include "FractureToolPlaneCut.generated.h"

class FFractureToolContext;
class UMeshOpPreviewWithBackgroundCompute;
struct FNoiseOffsets;
struct FNoiseSettings;

UCLASS(config = EditorPerProjectUserSettings)
class UFracturePlaneCutSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFracturePlaneCutSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, NumberPlanarCuts(1) {}

	/** Number of cutting planes. Only used when "Use Gizmo" is disabled */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Number of Cuts", UIMin = "1", UIMax = "20", ClampMin = "1", EditCondition = "bCanCutWithMultiplePlanes", HideEditConditionToggle))
	int32 NumberPlanarCuts;

	UPROPERTY()
	bool bCanCutWithMultiplePlanes = false;

};


UCLASS(DisplayName = "Plane Cut Tool", Category = "FractureTools")
class UFractureToolPlaneCut : public UFractureToolCutterBase, public UE::Geometry::IDynamicMeshOperatorFactory
{
public:
	GENERATED_BODY()

	UFractureToolPlaneCut(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void SelectedBonesChanged() override;


	virtual void OnTick(float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Shutdown() override;

	// IDynamicMeshOperatorFactory API, for generating noise preview meshes
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	virtual void ConfigureCutterSettings() override
	{
		CutterSettings->bGroupFractureToggleEnabled = true;
		CutterSettings->bDrawSitesToggleEnabled = false;
		CutterSettings->bNoisePreviewHasScale = true;
	}

protected:
	virtual void ClearVisualizations() override;

	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

private:
	// Slicing
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFracturePlaneCutSettings> PlaneCutSettings;

	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureTransformGizmoSettings> GizmoSettings;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> NoisePreview;

	void GenerateSliceTransforms(const FFractureToolContext& Context, TArray<FTransform>& CuttingPlaneTransforms);

	float RenderCuttingPlaneSize;
	const float GizmoPlaneSize = 100.f;
	TArray<FTransform> RenderCuttingPlanesTransforms;
	TArray<FNoiseOffsets> NoiseOffsets;
	TArray<FVector> NoisePivots;
	FVisualizationMappings PlanesMappings;
	float NoisePreviewExplodeAmount = 0;
};


