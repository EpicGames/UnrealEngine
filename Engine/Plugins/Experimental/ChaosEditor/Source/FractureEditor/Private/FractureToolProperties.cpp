// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolProperties.h"

#include "SGeometryCollectionOutliner.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolProperties)


#define LOCTEXT_NAMESPACE "FractureProperties"


UFractureToolSetInitialDynamicState::UFractureToolSetInitialDynamicState(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	StateSettings = NewObject<UFractureInitialDynamicStateSettings>(GetTransientPackage(), UFractureInitialDynamicStateSettings::StaticClass());
	StateSettings->OwnerTool = this;
}

FText UFractureToolSetInitialDynamicState::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSetInitialDynamicState", "State"));
}

FText UFractureToolSetInitialDynamicState::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSetInitialDynamicStateToolTip", \
		"Override initial dynamic state for selected bones. If the component's Object Type is set to Dynamic, the solver will use this override state instead. Setting a bone to Kinematic will have the effect of anchoring the bone to world space, for instance."));
}

FText UFractureToolSetInitialDynamicState::GetApplyText() const
{
	return FText(NSLOCTEXT("Fracture", "ExecuteSetInitialDynamicState", "Set State"));
}

FSlateIcon UFractureToolSetInitialDynamicState::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SetInitialDynamicState");
}

TArray<UObject*> UFractureToolSetInitialDynamicState::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(StateSettings);
	return Settings;
}

void UFractureToolSetInitialDynamicState::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SetInitialDynamicState", "State", \
		"Override initial dynamic state for selected bones. If the component's Object Type is set to Dynamic, the solver will use this override state instead. Setting a bone to Kinematic will have the effect of anchoring the bone to world space, for instance.", \
		EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->SetInitialDynamicState = UICommandInfo;
}

void UFractureToolSetInitialDynamicState::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	if (InToolkit.IsValid())
	{
		InToolkit.Pin()->SetOutlinerColumnMode(EOutlinerColumnMode::State);
	}
}

void UFractureToolSetInitialDynamicState::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SetSelectedInitialDynamicState(static_cast<int32>(StateSettings->InitialDynamicState));
		InToolkit.Pin()->RefreshOutliner();
	}
}

void UFractureToolSetInitialDynamicState::SetSelectedInitialDynamicState(int32 InitialDynamicState)
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FGeometryCollectionEdit GCEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysics, true /*bShapeIsUnchanged*/);
		if (UGeometryCollection* GCObject = GCEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GCObject->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(*GeometryCollection);
				if (ensure(AnchoringFacade.HasInitialDynamicStateAttribute()))
				{
					AnchoringFacade.SetInitialDynamicState(GeometryCollectionComponent->GetSelectedBones(), static_cast<Chaos::EObjectStateType>(InitialDynamicState));
				}
			}
		}
	}
}

//==============================================================================================================

UFractureToolSetRemoveOnBreak::UFractureToolSetRemoveOnBreak(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	RemoveOnBreakSettings = NewObject<UFractureRemoveOnBreakSettings>(GetTransientPackage(), UFractureRemoveOnBreakSettings::StaticClass());
	RemoveOnBreakSettings->OwnerTool = this;
}

FText UFractureToolSetRemoveOnBreak::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSetRemoveOnBreak", "Set Remove-On-Break"));
}

FText UFractureToolSetRemoveOnBreak::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSetRemoveOnBreakToolTip", "Set removal-on-break parameters"));
}

FText UFractureToolSetRemoveOnBreak::GetApplyText() const
{
	return FText(NSLOCTEXT("Fracture", "ExecuteSetRemoveOnBreak", "Apply removal-on-break parameters"));
}

FSlateIcon UFractureToolSetRemoveOnBreak::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SetRemoveOnBreak");
}

TArray<UObject*> UFractureToolSetRemoveOnBreak::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(RemoveOnBreakSettings);
	return Settings;
}

void UFractureToolSetRemoveOnBreak::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SetRemoveOnBreak", "Set Remove On Break", "Set removal-on-break parameters.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->SetRemoveOnBreak = UICommandInfo;
}

void UFractureToolSetRemoveOnBreak::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	if (InToolkit.IsValid())
	{
		InToolkit.Pin()->SetOutlinerColumnMode(EOutlinerColumnMode::Removal);
	}
}

void UFractureToolSetRemoveOnBreak::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Toolkit = InToolkit;
	if (InToolkit.IsValid())
	{
	 	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	 	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	 	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	 	{
	 		FGeometryCollectionEdit GCEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysics, true /*bShapeIsUnchanged*/);
	 		if (UGeometryCollection* GCObject = GCEdit.GetRestCollection())
	 		{
	 			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GCObject->GetGeometryCollection();
	 			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
	 			{
	 				TArray<int32> SelectedBones = GeometryCollectionComponent->GetSelectedBones();
	 				if (SelectedBones.Num())
	 				{
						GeometryCollection::Facades::FCollectionRemoveOnBreakFacade RemoveOnBreakFacade(*GeometryCollection);
						RemoveOnBreakFacade.DefineSchema();
	 					
						GeometryCollection::Facades::FRemoveOnBreakData RemoveOnBreakData;
						RemoveOnBreakData.SetBreakTimer(RemoveOnBreakSettings->PostBreakTimer.X, RemoveOnBreakSettings->PostBreakTimer.Y);
						RemoveOnBreakData.SetRemovalTimer(RemoveOnBreakSettings->RemovalTimer.X, RemoveOnBreakSettings->RemovalTimer.Y);
						RemoveOnBreakData.SetEnabled(RemoveOnBreakSettings->Enabled);
						RemoveOnBreakData.SetClusterCrumbling(RemoveOnBreakSettings->ClusterCrumbling);

						RemoveOnBreakFacade.SetFromIndexArray(SelectedBones, RemoveOnBreakData);
	 				}
	 			}
	 		}
	 	}
		InToolkit.Pin()->RefreshOutliner();
	}
}

void UFractureToolSetRemoveOnBreak::DeleteRemoveOnBreakData()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FGeometryCollectionEdit GCEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysics, true /*bShapeIsUnchanged*/);
		if (UGeometryCollection* GCObject = GCEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GCObject->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				GeometryCollection::Facades::FCollectionRemoveOnBreakFacade RemoveOnBreakFacade(*GeometryCollection);
				RemoveOnBreakFacade.RemoveSchema();
			}
		}
	}
	if (Toolkit.IsValid())
	{
		Toolkit.Pin()->RefreshOutliner();
	}
}


void UFractureRemoveOnBreakSettings::DeleteRemoveOnBreakData()
{
	UFractureToolSetRemoveOnBreak* RemoveOnBreakTool = Cast<UFractureToolSetRemoveOnBreak>(OwnerTool.Get());
	RemoveOnBreakTool->DeleteRemoveOnBreakData();
}

#undef LOCTEXT_NAMESPACE

