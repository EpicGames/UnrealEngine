// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "FractureTool.h"

#include "FractureToolProperties.generated.h"


UENUM(BlueprintType)
enum class EDynamicStateOverrideEnum : uint8
{
	NoOverride = 0										  UMETA(DisplayName = "No Override"),
	//~ Note: Sleeping is tagged as Hidden because it can't be set as Initial Dynamic State
	Sleeping = 1 /*~Chaos::EObjectStateType::Sleeping */    UMETA(Hidden, DisplayName = "Sleeping"),
	Kinematic = 2 /*~Chaos::EObjectStateType::Kinematic*/  UMETA(DisplayName = "Kinematic"),
	Static = 3    /*~Chaos::EObjectStateType::Static*/     UMETA(DisplayName = "Static")
};

/** Settings specifically related to the one-time destructive fracturing of a mesh **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureInitialDynamicStateSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureInitialDynamicStateSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, InitialDynamicState(EDynamicStateOverrideEnum::Kinematic)
	{}

	/** Simulation state to be set on selected bones */
	UPROPERTY(EditAnywhere, Category = SetInitialDynamicState, meta = (DisplayName = "Initial Dynamic State"))
	EDynamicStateOverrideEnum InitialDynamicState;

};


UCLASS(DisplayName = "State", Category = "FractureTools")
class UFractureToolSetInitialDynamicState : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolSetInitialDynamicState(const FObjectInitializer& ObjInit);

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	UPROPERTY(EditAnywhere, Category = InitialDynamicState)
	TObjectPtr<UFractureInitialDynamicStateSettings> StateSettings;

	static void SetSelectedInitialDynamicState(int32 InitialDynamicState); 
};

UCLASS(config = EditorPerProjectUserSettings)
class UFractureRemoveOnBreakSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureRemoveOnBreakSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, Enabled(true)
		, PostBreakTimer(0,0)
		, ClusterCrumbling(false)
		, RemovalTimer(2,3)
	{}

	/** whether or not the remove on fracture is enabled */
	UPROPERTY(EditAnywhere, Category = SetRemoveOnBreak, meta = (DisplayName = "Enabled"))
	bool Enabled;

	/** Min/Max time after break before removal starts */
	UPROPERTY(EditAnywhere, Category = SetRemoveOnBreak, meta = (DisplayName = "Post Break Timer", EditCondition="Enabled"))
	FVector2f PostBreakTimer;  

	/** When set, clusters will crumble when post break timer expires, non clusters will simply use the removal timer */
	UPROPERTY(EditAnywhere, Category = SetRemoveOnBreak, meta = (DisplayName = "Cluster Crumbling", EditCondition="Enabled"))
	bool ClusterCrumbling;

	/** Min/Max time for how long removal lasts - not applicable when cluster crumbling is on  */
	UPROPERTY(EditAnywhere, Category = SetRemoveOnBreak, meta = (DisplayName = "Removal Timer", EditCondition="!ClusterCrumbling && Enabled"))
	FVector2f RemovalTimer;
	
	/** remove the remove on break attribute from the geometry collection, usefull to save memory on the asset if not needed */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Delete Remove-On-Break Data"))
	void DeleteRemoveOnBreakData();
};

UCLASS(DisplayName = "RemoveOnBreak", Category = "FractureTools")
class UFractureToolSetRemoveOnBreak : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolSetRemoveOnBreak(const FObjectInitializer& ObjInit);

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	void DeleteRemoveOnBreakData();
	
	UPROPERTY(EditAnywhere, Category = InitialDynamicState)
	TObjectPtr<UFractureRemoveOnBreakSettings> RemoveOnBreakSettings;
	
private:
	TWeakPtr<FFractureEditorModeToolkit> Toolkit;
};