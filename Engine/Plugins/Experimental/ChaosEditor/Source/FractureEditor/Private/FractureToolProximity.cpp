// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolProximity.h"

#include "FractureToolContext.h"
#include "FractureModeSettings.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolProximity)

#define LOCTEXT_NAMESPACE "FractureToolProximity"


void UFractureProximityActions::SaveAsDefaults()
{
	UFractureToolProximity* ProximityTool = Cast<UFractureToolProximity>(OwnerTool.Get());
	UFractureModeSettings* ModeSettings = GetMutableDefault<UFractureModeSettings>();
	ModeSettings->Modify();
	ModeSettings->ProximityMethod = ProximityTool->ProximitySettings->Method;
	ModeSettings->ProximityDistanceThreshold = ProximityTool->ProximitySettings->DistanceThreshold;
	ModeSettings->bProximityUseAsConnectionGraph = ProximityTool->ProximitySettings->bUseAsConnectionGraph;
	ModeSettings->ProximityContactThreshold = ProximityTool->ProximitySettings->ContactThreshold;
	ModeSettings->ProximityContactMethod = ProximityTool->ProximitySettings->ContactMethod;
	ModeSettings->ProximityConnectionContactAreaMethod = ProximityTool->ProximitySettings->ContactAreaMethod;
}

void UFractureProximityActions::SetFromDefaults()
{
	UFractureToolProximity* ProximityTool = Cast<UFractureToolProximity>(OwnerTool.Get());
	const UFractureModeSettings* ModeSettings = GetDefault<UFractureModeSettings>();
	ProximityTool->ProximitySettings->Method = ModeSettings->ProximityMethod;
	ProximityTool->ProximitySettings->DistanceThreshold = ModeSettings->ProximityDistanceThreshold;
	ProximityTool->ProximitySettings->bUseAsConnectionGraph = ModeSettings->bProximityUseAsConnectionGraph;
	ProximityTool->ProximitySettings->ContactThreshold = ModeSettings->ProximityContactThreshold;
	ProximityTool->ProximitySettings->ContactMethod = ModeSettings->ProximityContactMethod;
	ProximityTool->ProximitySettings->ContactAreaMethod = ModeSettings->ProximityConnectionContactAreaMethod;

	ProximityTool->NotifyOfPropertyChangeByTool(this);
}



UFractureToolProximity::UFractureToolProximity(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	ProximitySettings = NewObject<UFractureProximitySettings>(GetTransientPackage(), UFractureProximitySettings::StaticClass());
	ProximitySettings->OwnerTool = this;
	ProximityActions = NewObject<UFractureProximityActions>(GetTransientPackage(), UFractureProximityActions::StaticClass());
	ProximityActions->OwnerTool = this;
}

bool UFractureToolProximity::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolProximity::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolProximity", "Update Proximity Generation Settings"));
}

FText UFractureToolProximity::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolProximityTooltip", "This tool visualizes and updates settings for proximity (contact) graph generation on the geometry collections"));
}

FSlateIcon UFractureToolProximity::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Proximity");
}

void UFractureToolProximity::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "Proximity", "Prxmty", "Update (and visualize) the proximity (contact) graph for the bones of geometry collections.", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->Proximity = UICommandInfo;
}

TArray<UObject*> UFractureToolProximity::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(ProximitySettings);
	Settings.Add(ProximityActions);
	return Settings;
}

void UFractureToolProximity::FractureContextChanged()
{
	UpdateVisualizations();
}

void UFractureToolProximity::UpdateVisualizations()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		if (FractureContext.GetGeometryCollection())
		{
			FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

			if (!Collection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
			{
				continue;
			}
			const TManagedArray<TSet<int32>>& Proximity = Collection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

			// Save the geometry collection component for rendering
			int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
			FCollectionVisInfo& Vis = ProximityVisualizations.Emplace_GetRef();
			Vis.CollectionIndex = CollectionIdx;

			// Get proximity graph node positions as bounding box centers for each piece of geometry
			Vis.GeoCenters.SetNum(Collection.NumElements(FGeometryCollection::GeometryGroup));
			TArray<FTransform> GlobalTransformArray;
			GeometryCollectionAlgo::GlobalMatrices(Collection.Transform, Collection.Parent, GlobalTransformArray);
			for (int32 GeoIdx = 0; GeoIdx < Collection.NumElements(FGeometryCollection::GeometryGroup); ++GeoIdx)
			{
				int32 TransformIdx = Collection.TransformIndex[GeoIdx];
				FVector BoxCenter = GlobalTransformArray[TransformIdx].TransformPosition(Collection.BoundingBox[GeoIdx].GetCenter());
				Vis.GeoCenters[GeoIdx] = BoxCenter;
			}

			// Collect relevant proximity graph edges to render
			auto AddEdgesForGeoIdx = [&Proximity, &Vis](int32 GeoIdx, bool bAddAll)
				{
					for (int32 GeoNbr : Proximity[GeoIdx])
					{
						if (bAddAll || GeoIdx < GeoNbr)
						{
							Vis.ProximityEdges.Add(FEdgeVisInfo{ GeoIdx, GeoNbr });
						}
					}
				};
			if (ProximitySettings->bOnlyShowForSelected)
			{
				TArray<int32> SelectedRigids;
				for (int32 TransformIdx : FractureContext.GetSelection())
				{
					SelectedRigids.Reset();
					FGeometryCollectionClusteringUtility::GetLeafBones(&Collection, TransformIdx, true, SelectedRigids);
					for (int32 RigidIdx : SelectedRigids)
					{
						int32 GeoIdx = Collection.TransformToGeometryIndex[RigidIdx];
						if (GeoIdx != INDEX_NONE)
						{
							AddEdgesForGeoIdx(GeoIdx, true);
						}
					}
				}
			}
			else
			{
				for (int32 GeoIdx = 0; GeoIdx < Collection.NumElements(FGeometryCollection::GeometryGroup); ++GeoIdx)
				{
					AddEdgesForGeoIdx(GeoIdx, false);
				}
			}
		}
	}
}

void UFractureToolProximity::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (ProximitySettings->bShowProximity)
	{
		for (const FCollectionVisInfo& Vis : ProximityVisualizations)
		{
			const FGeometryCollection& Collection = *VisualizedCollections[Vis.CollectionIndex]->GetRestCollection()->GetGeometryCollection();
			const TManagedArray<FVector3f>* ExplodedVectors = Collection.FindAttributeTyped<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);
			auto GetExplodedOffset = [&ExplodedVectors, &Collection](int32 GeoIdx)
			{
				return ExplodedVectors ?
					(FVector)(*ExplodedVectors)[Collection.TransformIndex[GeoIdx]] :
					FVector::ZeroVector;
			};
			const FTransform WorldTransform = VisualizedCollections[Vis.CollectionIndex]->GetComponentTransform();
			for (const FEdgeVisInfo& Edge : Vis.ProximityEdges)
			{
				FVector P1 = WorldTransform.TransformPosition(Vis.GeoCenters[Edge.A] + GetExplodedOffset(Edge.A));
				FVector P2 = WorldTransform.TransformPosition(Vis.GeoCenters[Edge.B] + GetExplodedOffset(Edge.B));
				PDI->DrawLine(P1, P2, ProximitySettings->LineColor, SDPG_Foreground, ProximitySettings->LineThickness, 0.001f);
			}

			for (int32 CenterIndex = 0; CenterIndex < Vis.GeoCenters.Num(); CenterIndex++)
			{
				// Draw centers for geometry (clusters can have geometry but it is not visible and won't be connected in the proximity graph)
				if (Collection.SimulationType[Collection.TransformIndex[CenterIndex]] == FGeometryCollection::ESimulationTypes::FST_Rigid)
				{
					FVector P = WorldTransform.TransformPosition(Vis.GeoCenters[CenterIndex] + GetExplodedOffset(CenterIndex));
					PDI->DrawPoint(P, ProximitySettings->CenterColor, ProximitySettings->CenterSize, SDPG_Foreground);
				}
			}
		}
	}
}

void UFractureToolProximity::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UFractureProximitySettings, bOnlyShowForSelected))
	{
		UpdateVisualizations();
	}
}

TArray<FFractureToolContext> UFractureToolProximity::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			// Generate a context for each selected node
			FFractureToolContext FullSelection(GeometryCollectionComponent);

			Contexts.Add(FullSelection);
		}
	}

	return Contexts;
}

int32 UFractureToolProximity::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.GetGeometryCollection().IsValid())
	{
		FGeometryCollectionProximityPropertiesInterface::FProximityProperties Properties = FractureContext.GetGeometryCollection()->GetProximityProperties();
		
		Properties.Method = ProximitySettings->Method;
		Properties.DistanceThreshold = ProximitySettings->DistanceThreshold;
		Properties.bUseAsConnectionGraph = ProximitySettings->bUseAsConnectionGraph;
		Properties.ContactAreaMethod = ProximitySettings->ContactAreaMethod;
		Properties.ContactMethod = ProximitySettings->ContactMethod;
		Properties.RequireContactAmount = ProximitySettings->ContactThreshold;
		
		FractureContext.GetGeometryCollection()->SetProximityProperties(Properties);

		// Invalidate proximity
		FGeometryCollectionProximityUtility ProximityUtility(FractureContext.GetGeometryCollection().Get());
		ProximityUtility.InvalidateProximity();
	}

	return INDEX_NONE;
}

void UFractureToolProximity::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
}

#undef LOCTEXT_NAMESPACE
