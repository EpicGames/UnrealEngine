// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include "FractureToolProximity.generated.h"

class FFractureToolContext;

/** Settings controlling how proximity is detected for geometry collections */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureProximitySettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureProximitySettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	// Which method to use to decide whether a given piece of geometry is in proximity with another
	UPROPERTY(EditAnywhere, Category = Automatic)
	EProximityMethod Method = EProximityMethod::Precise;

	// If hull-based proximity detection is enabled, amount to expand hulls when searching for overlapping neighbors
	UPROPERTY(EditAnywhere, Category = Automatic, meta = (ClampMin = "0", 
		EditCondition = "Method == EProximityMethod::ConvexHull || ContactMethod == EProximityContactMethod::ConvexHullSharpContact || ContactMethod == EProximityContactMethod::ConvexHullAreaContact || ContactAreaMethod == EConnectionContactMethod::ConvexHullContactArea"))
	double DistanceThreshold = 1;

	// Method to use to determine the contact between two pieces, if the Contact Threshold is greater than 0
	UPROPERTY(EditAnywhere, Category = Automatic)
	EProximityContactMethod ContactMethod = EProximityContactMethod::MinOverlapInProjectionToMajorAxes;

	// If greater than zero, proximity will be additionally filtered by a 'contact' threshold, in cm, to exclude grazing / corner proximity
	UPROPERTY(EditAnywhere, Category = Automatic, meta = (ClampMin = "0"))
	double ContactThreshold = 0;

	// Whether to automatically transform the proximity graph into a connection graph to be used for simulation
	UPROPERTY(EditAnywhere, Category = Automatic)
	bool bUseAsConnectionGraph = false;

	// Method to use for determining contact areas that define the strength of connections for destruction simulation
	UPROPERTY(EditAnywhere, Category = Automatic)
	EConnectionContactMethod ContactAreaMethod = EConnectionContactMethod::None;

	// Whether to display the proximity graph edges
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowProximity = true;

	// Whether to only show the proximity graph connections for selected bones
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (EditCondition = "bShowProximity"))
	bool bOnlyShowForSelected = false;

	// Line thickness for connections
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LineThickness = 0.5;

	// Line color for connections
	UPROPERTY(EditAnywhere, Category = Visualization)
	FColor LineColor = FColor::Yellow;

	// Point size for centers
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0", ClampMax = "20.0"))
	float CenterSize = 8.f;

	// Point color for centers
	UPROPERTY(EditAnywhere, Category = Visualization)
	FColor CenterColor = FColor::Blue;
};


/**
 * UFUNCTION actions to manage default proximity detection settings for geometry collections
 * (These are pulled out from the above settings object mainly to control their ordering in the properties panel)
 */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureProximityActions : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureProximityActions(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Save settings as project defaults, to be used for all new geometry collections */
	UFUNCTION(CallInEditor, Category = ProjectDefaults, meta = (DisplayName = "Save As Defaults"))
	void SaveAsDefaults();

	/** Set settings from current project defaults */
	UFUNCTION(CallInEditor, Category = ProjectDefaults, meta = (DisplayName = "Set From Defaults"))
	void SetFromDefaults();

};


UCLASS(DisplayName = "Proximity Tool", Category = "FractureTools")
class UFractureToolProximity : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolProximity(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureProximity", "ExecuteProximity", "Update Bone Proximity")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Gets the UI command info for this command */
	const TSharedPtr<FUICommandInfo>& GetUICommandInfo() const;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

	UPROPERTY(EditAnywhere, Category = Proximity)
	TObjectPtr<UFractureProximitySettings> ProximitySettings;

private:

	UPROPERTY(EditAnywhere, Category = Proximity)
	TObjectPtr<UFractureProximityActions> ProximityActions;

	void UpdateVisualizations();
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		ProximityVisualizations.Empty();
	}

	struct FEdgeVisInfo
	{
		int32 A, B;
	};

	struct FCollectionVisInfo
	{
		TArray<FEdgeVisInfo> ProximityEdges;
		TArray<FVector> GeoCenters;
		int32 CollectionIndex;
	};

	TArray<FCollectionVisInfo> ProximityVisualizations;

};
