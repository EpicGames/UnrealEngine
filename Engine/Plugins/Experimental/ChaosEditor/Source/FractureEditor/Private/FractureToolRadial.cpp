// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolRadial.h"

#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolRadial)


#define LOCTEXT_NAMESPACE "FractureRadial"


UFractureToolRadial::UFractureToolRadial(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	RadialSettings = NewObject<UFractureRadialSettings>(GetTransientPackage(), UFractureRadialSettings::StaticClass());
	RadialSettings->OwnerTool = this;
	GizmoSettings = NewObject<UFractureTransformGizmoSettings>(GetTransientPackage(), UFractureTransformGizmoSettings::StaticClass());
	GizmoSettings->OwnerTool = this;
}

void UFractureToolRadial::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	GizmoSettings->Setup(this);
}


void UFractureToolRadial::Shutdown()
{
	Super::Shutdown();
	GizmoSettings->Shutdown();
}

FText UFractureToolRadial::GetDisplayText() const
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolRadial", "Radial Voronoi Fracture")); 
}

FText UFractureToolRadial::GetTooltipText() const 
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolRadialTooltip", "Radial Voronoi Fracture create a radial distribution of Voronoi cells from a center point (for example, a wrecking ball crashing into a wall).  Click the Fracture Button to commit the fracture to the geometry collection."));
}

FSlateIcon UFractureToolRadial::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Radial");
}

void UFractureToolRadial::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Radial", "Radial", "Fracture using a Voronoi diagram with a circular pattern emanating from a center point.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Radial = UICommandInfo;
}

TArray<UObject*> UFractureToolRadial::GetSettingsObjects() const 
{ 
	TArray<UObject*> Settings;
	Settings.Add(RadialSettings);
	Settings.Add(GizmoSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

void UFractureToolRadial::GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites)
{
	const FVector::FReal AngularStep = 2 * PI / RadialSettings->AngularSteps;

	FBox Bounds = Context.GetWorldBounds();
	FVector Center(Bounds.GetCenter() + RadialSettings->Center);

	FRandomStream RandStream(Context.GetSeed());
	FVector UpVector(RadialSettings->Normal);
	UpVector.Normalize();
	FVector BasisX, BasisY;
	UpVector.FindBestAxisVectors(BasisX, BasisY);

	if (GizmoSettings->IsGizmoEnabled())
	{
		const FTransform& Transform = GizmoSettings->GetTransform();
		UpVector = Transform.GetUnitAxis(EAxis::Z);
		BasisX = Transform.GetUnitAxis(EAxis::X);
		BasisY = Transform.GetUnitAxis(EAxis::Y);
		Center = Transform.GetTranslation();
	}

	// Precompute consistent noise for each angular step
	TArray<FVector::FReal> AngleStepOffsets;
	AngleStepOffsets.SetNumUninitialized(RadialSettings->AngularSteps);
	for (int32 AngleIdx = 0; AngleIdx < RadialSettings->AngularSteps; ++AngleIdx)
	{
		AngleStepOffsets[AngleIdx] = FMath::DegreesToRadians(RandStream.FRandRange(-1, 1) * RadialSettings->AngularNoise);
	}

	// Compute radial positions following an (idx+1)^exp curve, and then re-normalize back to the Radius range
	TArray<FVector::FReal> RadialPositions;
	RadialPositions.SetNumUninitialized(RadialSettings->RadialSteps);
	FVector::FReal StepOffset = 0;
	for (int32 RadIdx = 0; RadIdx < RadialSettings->RadialSteps; ++RadIdx)
	{
		FVector::FReal RadialPos = FMath::Pow(RadIdx + 1, RadialSettings->RadialStepExponent) + StepOffset;
		if (RadIdx == 0)
		{
			// Note we bring the first point a half-step toward the center, and shift all subsequent points accordingly
			// so that for Exponent==1, the step from center to first boundary is the same distance as the step between each boundary
			// (this is only necessary because there is no Voronoi site at the center)
			RadialPos *= .5;
			StepOffset = -RadialPos;
		}

		RadialPositions[RadIdx] = RadialPos;
	}
	// Normalize positions so that the diagram fits in the target radius
	FVector::FReal RadialPosNorm = RadialSettings->Radius / RadialPositions.Last();
	for (FVector::FReal& RadialPos : RadialPositions)
	{
		RadialPos = RadialPos * RadialPosNorm;
	}
	// Add radial noise 
	for (int32 RadIdx = 0; RadIdx < RadialSettings->RadialSteps; ++RadIdx)
	{
		FVector::FReal& RadialPos = RadialPositions[RadIdx];
		// Offset by RadialNoise, but don't allow noise to take the value below 0
		RadialPos += RandStream.FRandRange(-FMath::Min(RadialPos, RadialSettings->RadialNoise), RadialSettings->RadialNoise);
	}
	// make sure the positions remain in increasing order
	RadialPositions.Sort();
	// Adjust positions so they are never closer than the RadialMinStep
	FVector::FReal LastRadialPos = 0;
	for (int32 RadIdx = 0; RadIdx < RadialSettings->RadialSteps; ++RadIdx)
	{
		FVector::FReal MinStep = RadialSettings->RadialMinStep;
		if (RadIdx == 0)
		{
			MinStep *= .5;
		}
		if (RadialPositions[RadIdx] - LastRadialPos < MinStep)
		{
			RadialPositions[RadIdx] = LastRadialPos + MinStep;
		}
		LastRadialPos = RadialPositions[RadIdx];
	}

	// Add a bit of noise to work around failure case in Voro++
	// TODO: fix the failure case in Voro++ and remove this
	float MinRadialVariability = RadialSettings->Radius > 1.f ? .0001f : 0.f;
	float UseRadialVariability = FMath::Max(MinRadialVariability, RadialSettings->RadialVariability);

	// Create the radial Voronoi sites
	for (int32 ii = 0; ii < RadialSettings->RadialSteps; ++ii)
	{
		FVector::FReal Len = RadialPositions[ii];
		FVector::FReal Angle = FMath::DegreesToRadians(RadialSettings->AngleOffset);
		for (int32 kk = 0; kk < RadialSettings->AngularSteps; ++kk, Angle += AngularStep)
		{
			// Add the global noise and the per-point noise into the angle
			FVector::FReal UseAngle = Angle + AngleStepOffsets[kk] + FMath::DegreesToRadians(RandStream.FRand() * RadialSettings->AngularVariability);
			// Add per point noise into the radial position
			FVector::FReal UseRadius = Len + FVector::FReal(RandStream.FRand() * UseRadialVariability);
			FVector RotatingOffset = UseRadius * (FMath::Cos(UseAngle) * BasisX + FMath::Sin(UseAngle) * BasisY);
			Sites.Emplace(Center + RotatingOffset + UpVector * (RandStream.FRandRange(-1, 1) * RadialSettings->AxialVariability));
		}
	}
}

void UFractureToolRadial::SelectedBonesChanged()
{
	GizmoSettings->ResetGizmo();
	Super::SelectedBonesChanged();
}


#undef LOCTEXT_NAMESPACE

