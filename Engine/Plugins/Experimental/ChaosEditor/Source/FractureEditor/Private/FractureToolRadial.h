// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "FractureToolCutter.h"

#include "FractureToolRadial.generated.h"

class FFractureToolContext;

UCLASS(config = EditorPerProjectUserSettings)
class UFractureRadialSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureRadialSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, Center(FVector(0, 0, 0))
		, Normal(FVector(0, 0, 1))
	{}

	/** Center of generated pattern. Only used when "Use Gizmo" is disabled */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (EditCondition = "!bPositionedByGizmo", HideEditConditionToggle))
	FVector Center;

	/** Normal to plane in which sites are generated. Only used when "Use Gizmo" is disabled */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (EditCondition = "!bPositionedByGizmo", HideEditConditionToggle))
	FVector Normal;

	UPROPERTY()
	bool bPositionedByGizmo = true;

	/** Number of angular steps */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (DisplayName = "Angular Steps", UIMin = "1", UIMax = "50", ClampMin = "1"))
	int AngularSteps = 5;

	/** Angle offset at each radial step (in degrees) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (DisplayName = "Angle Offset"))
	float AngleOffset = 0.0f;

	/** Amount of global variation to apply to each angular step (in degrees) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi)
	float AngularNoise = 0.f;

	/** Pattern radius (in cm) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (DisplayName = "Radius", UIMin = "0.0", ClampMin = "0.0"))
	float Radius = 50.0f;

	/** Number of radial steps */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (DisplayName = "Radial Steps", UIMin = "1", UIMax = "50", ClampMin = "1"))
	int RadialSteps = 5;

	/** Radial steps will follow a distribution based on this exponent, i.e., Pow(distance from center, RadialStepExponent) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (UIMin = ".01", UIMax = "10", ClampMin = ".01", ClampMax = "20"))
	float RadialStepExponent = 1.f;

	/** Minimum radial separation between any two voronoi points (in cm) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (UIMin = ".25", ClampMin = ".01"))
	float RadialMinStep = 1.f;

	/** Amount of global variation to apply to each radial step (in cm) */
	UPROPERTY(EditAnywhere, Category = RadialVoronoi, meta = (ClampMin = "0"))
	float RadialNoise = 0.f;


	/** Amount to randomly displace each Voronoi site radially (in cm) */
	UPROPERTY(EditAnywhere, Category = PerPointVariability, meta = (ClampMin = "0.0"))
	float RadialVariability = 0.f;

	/** Amount to randomly displace each Voronoi site in angle (in degrees) */
	UPROPERTY(EditAnywhere, Category = PerPointVariability, meta = (ClampMin = "0.0"))
	float AngularVariability = 0.f;

	/** Amount to randomly displace each Voronoi site in the direction of the rotation axis (in cm) */
	UPROPERTY(EditAnywhere, Category = PerPointVariability, meta = (ClampMin = "0.0"))
	float AxialVariability = 0.f;
};


UCLASS(DisplayName="Radial Voronoi", Category="FractureTools")
class UFractureToolRadial : public UFractureToolVoronoiCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolRadial(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void SelectedBonesChanged() override;

	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext ) override;

	// Radial Voronoi Fracture Input Settings
	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureRadialSettings> RadialSettings;

	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureTransformGizmoSettings> GizmoSettings;

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Shutdown() override;

	virtual void UpdateUseGizmo(bool bUseGizmo) override
	{
		Super::UpdateUseGizmo(bUseGizmo);
		RadialSettings->bPositionedByGizmo = bUseGizmo;
		NotifyOfPropertyChangeByTool(RadialSettings);
	}

protected:
	void GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites) override;

};