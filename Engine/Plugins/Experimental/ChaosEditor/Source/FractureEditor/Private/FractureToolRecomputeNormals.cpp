// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolRecomputeNormals.h"
#include "FractureEditorStyle.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolRecomputeNormals)

#define LOCTEXT_NAMESPACE "FractureRecomputeNormals"


UFractureToolRecomputeNormals::UFractureToolRecomputeNormals(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	NormalsSettings = NewObject<UFractureRecomputeNormalsSettings>(GetTransientPackage(), UFractureRecomputeNormalsSettings::StaticClass());
	NormalsSettings->OwnerTool = this;
}

FText UFractureToolRecomputeNormals::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolRecomputeNormals", "Recompute Normals and Tangents"));
}

FText UFractureToolRecomputeNormals::GetTooltipText() const 
{
	return FText(LOCTEXT("FractureToolRecomputeNormalsTooltip", "The Recompute Normals tool recompute normals and tangents for selected geometry."));
}

FSlateIcon UFractureToolRecomputeNormals::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.RecomputeNormals");
}

void UFractureToolRecomputeNormals::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "RecomputeNormals", "Normals", "Recompute (and visualize) normals and tangents.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->RecomputeNormals = UICommandInfo;
}

void UFractureToolRecomputeNormals::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (NormalsSettings->bShowTangents || NormalsSettings->bShowNormals)
	{
		EnumerateVisualizationMapping(PointsMappings, DisplayVertices.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			FVector Point = DisplayVertices[Idx] + ExplodedVector;
			FVector Normal = DisplayNormals[Idx] * NormalsSettings->Length;
			FVector TanU = DisplayTanUs[Idx] * NormalsSettings->Length;
			FVector TanV = DisplayTanVs[Idx] * NormalsSettings->Length;
			if (NormalsSettings->bShowNormals)
			{
				PDI->DrawLine(Point, Point + Normal, FLinearColor::Red, SDPG_Foreground);
			}
			if (NormalsSettings->bShowTangents)
			{
				PDI->DrawLine(Point, Point + TanU, FLinearColor::Green, SDPG_Foreground);
				PDI->DrawLine(Point, Point + TanV, FLinearColor::Blue, SDPG_Foreground);
			}
		});
	}
}

TArray<UObject*> UFractureToolRecomputeNormals::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(NormalsSettings);
	return Settings;
}


void UFractureToolRecomputeNormals::FractureContextChanged()
{
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		

		FTransform OuterTransform = FractureContext.GetTransform();
		for (int32 TransformIdx : FractureContext.GetSelection())
		{
			int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
			PointsMappings.AddMapping(CollectionIdx, TransformIdx, DisplayVertices.Num());

			FTransform InnerTransform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TransformIdx);
			FTransform CombinedTransform = InnerTransform * OuterTransform;
			int32 GeometryIdx = Collection.TransformToGeometryIndex[TransformIdx];
			if (GeometryIdx > -1)
			{
				TArray<bool> ShowVerts;
				int32 VertStart = Collection.VertexStart[GeometryIdx];
				int32 VertCount = Collection.VertexCount[GeometryIdx];
				int32 FaceStart = Collection.FaceStart[GeometryIdx];
				int32 FaceCount = Collection.FaceCount[GeometryIdx];
				bool bShowAll = !NormalsSettings->bOnlyInternalSurfaces;
				ShowVerts.Init(bShowAll, VertCount);
				if (!bShowAll)
				{
					for (int FIdx = FaceStart; FIdx < FaceStart + FaceCount; FIdx++)
					{
						if (Collection.Visible[FIdx] && Collection.Internal[FIdx])
						{
							FIntVector Face = Collection.Indices[FIdx];
							ShowVerts[Face.X - VertStart] = true;
							ShowVerts[Face.Y - VertStart] = true;
							ShowVerts[Face.Z - VertStart] = true;
						}
					}
				}
				for (int32 VIdx = VertStart; VIdx < VertStart + VertCount; VIdx++)
				{
					if (ShowVerts[VIdx - VertStart])
					{
						DisplayVertices.Add(CombinedTransform.TransformPosition((FVector)Collection.Vertex[VIdx]));
						DisplayNormals.Add(CombinedTransform.TransformVectorNoScale((FVector)Collection.Normal[VIdx]));
						DisplayTanUs.Add(CombinedTransform.TransformVectorNoScale((FVector)Collection.TangentU[VIdx]));
						DisplayTanVs.Add(CombinedTransform.TransformVectorNoScale((FVector)Collection.TangentV[VIdx]));
					}
				}
			}
		}
	}
}

int32 UFractureToolRecomputeNormals::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		const UFractureCollisionSettings* LocalCutSettings = CollisionSettings;

		bool bOnlyInternalFaces = true;
		RecomputeNormalsAndTangents(NormalsSettings->bOnlyTangents, NormalsSettings->bRecomputeSharpEdges, NormalsSettings->SharpEdgeAngleThreshold, *FractureContext.GetGeometryCollection(), FractureContext.GetSelection(), NormalsSettings->bOnlyInternalSurfaces);
	}

	return INDEX_NONE;
}


#undef LOCTEXT_NAMESPACE
