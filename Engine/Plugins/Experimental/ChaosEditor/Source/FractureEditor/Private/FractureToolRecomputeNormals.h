// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "Engine/StaticMeshActor.h"

#include "FractureToolRecomputeNormals.generated.h"

class FFractureToolContext;

/** Settings for visualizing and recomputing normals and tangents */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureRecomputeNormalsSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureRecomputeNormalsSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	{}

	/** Whether to display normal vectors */
	UPROPERTY(EditAnywhere, Category = DisplaySettings)
	bool bShowNormals = true;

	/** Whether to display tangent and bitangent vectors */
	UPROPERTY(EditAnywhere, Category = DisplaySettings)
	bool bShowTangents = true;

	/** Length of display normal and tangent vectors */
	UPROPERTY(EditAnywhere, Category = DisplaySettings, meta = (ClampMin = ".001", ClampMax = "100", UIMax = "10"))
	float Length = 2.0f;

	/** Whether to only recompute tangents, and leave normals as they were */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings)
	bool bOnlyTangents = false;

	/** If true, update where edges are 'sharp' by comparing adjacent triangle face normals vs the Sharp Edge Angle Threshold. */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, meta = (EditCondition = "!bOnlyTangents"))
	bool bRecomputeSharpEdges = false;

	/** Threshold on angle of change in face normals across an edge, above which we create a sharp edge if bRecomputeSharpEdges is true */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, meta = (UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", EditCondition = "bRecomputeSharpEdges && !bOnlyTangents"))
	float SharpEdgeAngleThreshold = 60.0f;

	/** Whether to only change internal surface normals / tangents */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, AdvancedDisplay)
	bool bOnlyInternalSurfaces = true;
};


// Note this tool doesn't actually fracture, but it does remake pieces of geometry and shares a lot of machinery with the fracture tools
UCLASS(DisplayName = "Recompute Normals Tool", Category = "FractureTools")
class UFractureToolRecomputeNormals : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolRecomputeNormals(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureRecomputeNormals", "ExecuteRecomputeNormals", "Recompute")); }

	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool ExecuteUpdatesShape() const override
	{
		return false;
	}

protected:
	UPROPERTY()
	TObjectPtr<UFractureRecomputeNormalsSettings> NormalsSettings;

	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		DisplayVertices.Empty();
		DisplayNormals.Empty();
		DisplayTanUs.Empty();
		DisplayTanVs.Empty();
		PointsMappings.Empty();
	}

	TArray<FVector> DisplayVertices;
	TArray<FVector> DisplayNormals;
	TArray<FVector> DisplayTanUs;
	TArray<FVector> DisplayTanVs;
	FVisualizationMappings PointsMappings;

};


