// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolResample.h"
#include "FractureEditorStyle.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolResample)

#define LOCTEXT_NAMESPACE "FractureResample"


UFractureToolResample::UFractureToolResample(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	ResampleSettings = NewObject<UFractureResampleSettings>(GetTransientPackage(), UFractureResampleSettings::StaticClass());
	ResampleSettings->OwnerTool = this;
}

FText UFractureToolResample::GetDisplayText() const
{
	return FText(NSLOCTEXT("Resample", "FractureToolResample", "Update Collision Samples")); 
}

FText UFractureToolResample::GetTooltipText() const 
{
	return FText(NSLOCTEXT("Resample", "FractureToolResampleTooltip", "Resample to add collision particles in large flat regions that otherwise might have poor collision response."));
}

FSlateIcon UFractureToolResample::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Resample");
}

void UFractureToolResample::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Resample", "Resmpl", "Resample to add collision particles in large flat regions that otherwise might have poor collision response. Only useful to help improve Particle-Implicit collisions.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Resample = UICommandInfo;
}

void UFractureToolResample::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	EnumerateVisualizationMapping(PointsMappings, GeneratedPoints.Num(), [&](int32 Idx, FVector ExplodedVector)
	{
		FVector Point = GeneratedPoints[Idx];
		PDI->DrawPoint(Point + ExplodedVector, FLinearColor(1,0,1), 4.f, SDPG_Foreground);
	});
}

TArray<UObject*> UFractureToolResample::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(CollisionSettings);
	Settings.Add(ResampleSettings);
	return Settings;
}


void UFractureToolResample::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		

		FTransform OuterTransform = FractureContext.GetTransform();
		for (int32 TransformIdx : FractureContext.GetSelection())
		{
			int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
			PointsMappings.AddMapping(CollectionIdx, TransformIdx, GeneratedPoints.Num());

			FTransform InnerTransform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TransformIdx);
			FTransform CombinedTransform = InnerTransform * OuterTransform;
			int32 GeometryIdx = Collection.TransformToGeometryIndex[TransformIdx];
			int32 VertStart = Collection.VertexStart[GeometryIdx];
			int32 VertEnd = VertStart + Collection.VertexCount[GeometryIdx];
			int32 FaceStart = Collection.FaceStart[GeometryIdx];
			int32 FaceEnd = FaceStart + Collection.FaceCount[GeometryIdx];
			// only show off-vertex samples; skip over the samples that are on faces
			if (ResampleSettings->bOnlyShowAddedPoints)
			{
				for (int32 FIdx = FaceStart; FIdx < FaceEnd; FIdx++)
				{
					FIntVector Face = Collection.Indices[FIdx];
					VertStart = FMath::Max(VertStart, Face.GetMax() + 1);
				}
			}
			for (int32 VIdx = VertStart; VIdx < VertEnd; VIdx++)
			{
				GeneratedPoints.Add(CombinedTransform.TransformPosition((FVector)Collection.Vertex[VIdx]));
			}
		}
	}
}

int32 UFractureToolResample::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.IsValid())
	{
		return AddCollisionSampleVertices(CollisionSettings->GetPointSpacing(), *FractureContext.GetGeometryCollection(), FractureContext.GetSelection());
	}

	return INDEX_NONE;
}


#undef LOCTEXT_NAMESPACE
