// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "Engine/StaticMeshActor.h"

#include "FractureToolResample.generated.h"

class FFractureToolContext;

/** Settings giving additional control over geometry resampling + related visualization */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureResampleSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureResampleSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Whether to visualize all mesh vertices or only the ones added by resampling */
	UPROPERTY(EditAnywhere, Category = FilterSettings)
	bool bOnlyShowAddedPoints = false;

};

// Note this tool doesn't actually fracture, but it does remake pieces of geometry and shares a lot of machinery with the fracture tools
UCLASS(DisplayName = "Resample Tool", Category = "FractureTools")
class UFractureToolResample : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolResample(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("Resample", "ExecuteResample", "Resample")); }

	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool ExecuteUpdatesShape() const override
	{
		return false;
	}

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		GeneratedPoints.Empty();
		PointsMappings.Empty();
	}

private:


	UPROPERTY(EditAnywhere, Category = FixGeo)
	TObjectPtr<UFractureResampleSettings> ResampleSettings;

	TArray<FVector> GeneratedPoints;
	FVisualizationMappings PointsMappings;
};


