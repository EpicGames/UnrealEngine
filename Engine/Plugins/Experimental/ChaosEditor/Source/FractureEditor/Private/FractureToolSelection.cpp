// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolSelection.h"
#include "FractureEditorStyle.h"
#include "FractureSettings.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorModeManager.h"
#include "FractureEditorMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "GameFramework/Actor.h"
#include "Engine/Selection.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "UnrealClient.h"

#include "Spatial/PriorityOrderPoints.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolSelection)

#define LOCTEXT_NAMESPACE "FractureSelection"


namespace UE { namespace FractureToolSelectionInternal {

bool IsGeometryCollectionSelected()
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			TArray<UGeometryCollectionComponent*, TInlineAllocator<1>> GeometryComponents;
			Actor->GetComponents(GeometryComponents);
			if (GeometryComponents.Num() > 0)
			{
				return true;
			}
		}
	}
	return false;
}

}} // namespace UE::FractureToolSelectionInternal

UFractureToolSelection::UFractureToolSelection(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	SelectionSettings = NewObject<UFractureSelectionSettings>(GetTransientPackage(), UFractureSelectionSettings::StaticClass());
	SelectionSettings->OwnerTool = this;
}

FText UFractureToolSelection::GetDisplayText() const
{
	return FText(NSLOCTEXT("FractureSelection", "FractureToolSelection", "Interactive Selection Tool")); 
}

FText UFractureToolSelection::GetTooltipText() const 
{
	return FText(NSLOCTEXT("FractureSelection", "FractureToolSelectionTooltip", "The interactive selection mode provides useful options to help select bones."));
}

FSlateIcon UFractureToolSelection::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectCustom");
}

void UFractureToolSelection::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "SelectCustom", "Intrctve", "The interactive selection mode provides useful options to help select bones.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->SelectCustom = UICommandInfo;
}

void UFractureToolSelection::CreateRectangleSelectionBehavior()
{
	UFractureEditorMode* Mode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (!Mode)
	{
		return;
	}

	// Set things up for being able to add behaviors to live preview.
	SelectionBehaviorSet = NewObject<UInputBehaviorSet>();
	SelectionBehaviorSource = NewObject<ULocalInputBehaviorSource>();
	SelectionBehaviorSource->GetInputBehaviorsFunc = [this]() { return SelectionBehaviorSet; };

	RectangleMarqueeManager = NewObject<URectangleMarqueeManager>();
	RectangleMarqueeManager->bUseExternalClickDragBehavior = true;
	RectangleMarqueeManager->Setup([this](UInputBehavior* Behavior, void* Source)
	{
		SelectionBehaviorSet->Add(Behavior, Source);
	});
	RectangleMarqueeManager->OnDragRectangleStarted.AddUObject(this, &UFractureToolSelection::OnDragRectangleStarted);
	RectangleMarqueeManager->OnDragRectangleChanged.AddUObject(this, &UFractureToolSelection::OnDragRectangleChanged);
	RectangleMarqueeManager->OnDragRectangleFinished.AddUObject(this, &UFractureToolSelection::OnDragRectangleFinished);

	USingleClickOrDragInputBehavior* ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, RectangleMarqueeManager);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	SelectionBehaviorSet->Add(ClickOrDragBehavior, this);

	UsedToolsContext = Mode->GetInteractiveToolsContext();
	UsedToolsContext->InputRouter->RegisterSource(SelectionBehaviorSource);
}

void UFractureToolSelection::DestroyRectangleSelectionBehavior()
{
	if (SelectionBehaviorSource && UsedToolsContext)
	{
		UsedToolsContext->InputRouter->DeregisterSource(SelectionBehaviorSource);
	}

	UsedToolsContext = nullptr;
	SelectionBehaviorSet = nullptr;
	SelectionBehaviorSource = nullptr;
	RectangleMarqueeManager = nullptr;
}

void UFractureToolSelection::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case ShiftModifierID:
		bShiftToggle = bIsOn;
		break;
	case CtrlModifierID:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

// These get bound to the delegates on the marquee manager.
void UFractureToolSelection::OnDragRectangleStarted()
{
	// TODO: initialize a live preview of the selection change?
}

void UFractureToolSelection::OnDragRectangleChanged(const FCameraRectangle& Rectangle)
{
	// TODO: implement a live preview of the selection change?
}

void UFractureToolSelection::OnClicked(const FInputDeviceRay& ClickPos)
{
	UFractureEditorMode* Mode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (!Mode)
	{
		return;
	}
	FViewport* Viewport = Mode->GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!Viewport || !Viewport->HasFocus() || !ClickPos.bHas2D)
	{
		return;
	}
	TRefCountPtr<HHitProxy> HitProxy = Viewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y);
	Mode->SelectFromClick(HitProxy, bCtrlToggle, bShiftToggle);
}

void UFractureToolSelection::OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{
	if (bCancelled)
	{
		return;
	}

	UFractureEditorMode* FractureMode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (!FractureMode)
	{
		return;
	}

	FConvexVolume FrustumVolume = Rectangle.FrustumAsConvexVolume();

	bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			FractureMode->UpdateSelectionInFrustum(FrustumVolume, Actor, bStrictDragSelection, bShiftToggle, bCtrlToggle);
		}
	}
}

void URectangleMarqueeManager::Setup(TFunctionRef<void(UInputBehavior*, void*)> AddBehaviorFunc)
{
	if (bUseExternalClickDragBehavior == false)
	{
		ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
		ClickDragBehavior->SetDefaultPriority(BasePriority);
		ClickDragBehavior->Initialize(this);
		AddBehaviorFunc(ClickDragBehavior, this);
	}
	SetIsEnabled(true);
}

bool URectangleMarqueeManager::IsEnabled()
{
	return bIsEnabled;
}

void URectangleMarqueeManager::SetIsEnabled(bool bOn)
{
	if (bIsDragging && !bOn)
	{
		OnTerminateDragSequence();
	}

	bIsEnabled = bOn;
}

void URectangleMarqueeManager::SetBasePriority(const FInputCapturePriority& Priority)
{
	BasePriority = Priority;
	if (ClickDragBehavior)
	{
		ClickDragBehavior->SetDefaultPriority(Priority);
	}
}

FInputCapturePriority URectangleMarqueeManager::GetPriority() const
{
	return BasePriority;
}

void URectangleMarqueeManager::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId)->
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraRectangle.CameraState);
}

FInputRayHit URectangleMarqueeManager::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return bIsEnabled ?
		FInputRayHit(TNumericLimits<float>::Max()) : // bHit is true. Depth is max to lose the standard tiebreaker.
		FInputRayHit(); // bHit is false
}

void URectangleMarqueeManager::OnClickPress(const FInputDeviceRay& PressPos)
{
	if (!PressPos.bHas2D)
	{
		bIsDragging = false;
		return;
	}

	CameraRectangle.RectangleStartRay = PressPos;
	CameraRectangle.RectangleEndRay = PressPos;
	CameraRectangle.Initialize();

	bIsOnDragRectangleChangedDeferred = false;
	OnDragRectangleStarted.Broadcast();
}

void URectangleMarqueeManager::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!DragPos.bHas2D)
	{
		return;
	}

	bIsDragging = true;
	CameraRectangle.RectangleEndRay = DragPos;
	CameraRectangle.Initialize();

	if (bIsOnDragRectangleChangedDeferred)
	{
		return;
	}

	double Time = 0.0;
	FDurationTimer Timer(Time);
	OnDragRectangleChanged.Broadcast(CameraRectangle);
	Timer.Stop();

	if (Time > OnDragRectangleChangedDeferredThreshold)
	{
		bIsOnDragRectangleChangedDeferred = true;
	}
}

void URectangleMarqueeManager::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if (bIsOnDragRectangleChangedDeferred)
	{
		bIsOnDragRectangleChangedDeferred = false;
		OnClickDrag(ReleasePos);
	}

	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, false);
}

void URectangleMarqueeManager::OnTerminateDragSequence()
{
	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, true);
}


void URectangleMarqueeManager::DrawHUD(FCanvas* Canvas, bool bThisViewHasFocus)
{
	if (bThisViewHasFocus && bIsDragging)
	{
		FVector2D Start = CameraRectangle.RectangleStartRay.ScreenPosition;
		FVector2D Curr = CameraRectangle.RectangleEndRay.ScreenPosition;
		FCanvasBoxItem BoxItem(Start / Canvas->GetDPIScale(), (Curr - Start) / Canvas->GetDPIScale());
		BoxItem.SetColor(FLinearColor::White);
		Canvas->DrawItem(BoxItem);
	}
}


void UFractureToolSelection::Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	Super::Setup(InToolkit);
	if (InToolkit.IsValid())
	{
		InToolkit.Pin()->SetOutlinerColumnMode(EOutlinerColumnMode::Size);
	}
}

void UFractureToolSelection::Shutdown()
{
	DestroyRectangleSelectionBehavior();
	Super::Shutdown();
}

void UFractureToolSelection::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	Super::DrawHUD(ViewportClient, Viewport, View, Canvas);
	
	if (RectangleMarqueeManager)
	{
		const FEditorViewportClient* Focused = GLevelEditorModeTools().GetFocusedViewportClient();
		const bool bThisViewHasFocus = (ViewportClient == Focused);

		RectangleMarqueeManager->DrawHUD(Canvas, bThisViewHasFocus);
	}
}

void UFractureToolSelection::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (RectangleMarqueeManager)
	{
		RectangleMarqueeManager->Render(View, Viewport, PDI);
	}

	EnumerateVisualizationMapping(SelectionMappings, SelectionBounds.Num(), [&](int32 Idx, FVector ExplodedVector)
	{
		const FBox& Box = SelectionBounds[Idx];
		FVector B000 = Box.Min + ExplodedVector;
		FVector B111 = Box.Max + ExplodedVector;
		FVector B011(B000.X, B111.Y, B111.Z);
		FVector B101(B111.X, B000.Y, B111.Z);
		FVector B110(B111.X, B111.Y, B000.Z);
		FVector B001(B000.X, B000.Y, B111.Z);
		FVector B010(B000.X, B111.Y, B000.Z);
		FVector B100(B111.X, B000.Y, B000.Z);
		PDI->DrawLine(B000, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B000, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B000, B001, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B111, B110, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B001, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B001, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B110, B100, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B110, B010, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B100, B101, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
		PDI->DrawLine(B010, B011, FLinearColor::Red, SDPG_Foreground, 0.0f, 0.001f);
	});
}

TArray<UObject*> UFractureToolSelection::GetSettingsObjects() const
 {
	TArray<UObject*> Settings;
	Settings.Add(SelectionSettings);
	return Settings;
}

TArray<FBox> GetAllBounds(const FGeometryCollection& Collection, FTransform OuterTransform = FTransform::Identity)
{
	// Build bounding boxes for every transform (including clusters)
	TArray<FTransform> AllTransforms;
	TArray<FBox> AllBounds;
	int32 NumBones = Collection.NumElements(FGeometryCollection::TransformGroup);
	AllBounds.SetNum(NumBones);
	GeometryCollectionAlgo::GlobalMatrices(Collection.Transform, Collection.Parent, AllTransforms);
	FBox EmptyBounds(EForceInit::ForceInit);
	AllBounds.Init(EmptyBounds, NumBones);
	TArray<int32> BoneIndices = GeometryCollectionAlgo::ComputeRecursiveOrder(Collection);
	for (int32 BoneIdx : BoneIndices)
	{
		int32 GeoIdx = Collection.TransformToGeometryIndex[BoneIdx];
		if (GeoIdx != INDEX_NONE)
		{
			const FTransform& InnerTransform = AllTransforms[BoneIdx];
			FTransform CombinedTransform = InnerTransform * OuterTransform;

			AllBounds[BoneIdx] += Collection.BoundingBox[GeoIdx].TransformBy(CombinedTransform);
		}
		int32 ParentIdx = Collection.Parent[BoneIdx];
		if (ParentIdx != INDEX_NONE)
		{
			AllBounds[ParentIdx] += AllBounds[BoneIdx];
		}
	}
	return AllBounds;
}


void UFractureToolSelection::FractureContextChanged()
{
	bool bWantsRectangleSelection =
		SelectionSettings->MouseSelectionMethod == EMouseSelectionMethod::RectSelect &&
		UE::FractureToolSelectionInternal::IsGeometryCollectionSelected();
	if (!RectangleMarqueeManager && bWantsRectangleSelection)
	{
		CreateRectangleSelectionBehavior();
	}
	else if (RectangleMarqueeManager && !bWantsRectangleSelection)
	{
		DestroyRectangleSelectionBehavior();
	}

	if (RectangleMarqueeManager)
	{
		RectangleMarqueeManager->SetIsEnabled(bWantsRectangleSelection);
	}

	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	for (const FFractureToolContext& FractureContext : FractureContexts)
	{
		FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();
		int CollectionIdx = VisualizedCollections.Emplace(FractureContext.GetGeometryCollectionComponent());

		TArray<int32> SelectedIndices;
		if (!GetBonesByVolume(Collection, SelectedIndices))
		{
			continue;
		}

		FTransform OuterTransform = FractureContext.GetTransform();

		// Build bounding boxes for every transform (including clusters) in the global space of the component
		TArray<FBox> AllBounds = GetAllBounds(Collection, OuterTransform);

		// Add bounding boxes for the selected indices to the visualization
		for (int32 TransformIdx : SelectedIndices)
		{
			FBox Bounds = AllBounds[TransformIdx];
			SelectionMappings.AddMapping(CollectionIdx, TransformIdx, SelectionBounds.Num());
			SelectionBounds.Add(Bounds);
		}
	}
}

TInterval<float> UFractureToolSelection::GetVolumeRange(const TManagedArray<float>& Volumes, const TManagedArray<int32>& SimulationType) const
{
	check(Volumes.Num() == SimulationType.Num());

	TInterval<float> Range;
	if (SelectionSettings->VolumeSelectionMethod == EVolumeSelectionMethod::CubeRootOfVolume)
	{
		Range.Min = SelectionSettings->MinVolume * VolDimScale;
		Range.Max = SelectionSettings->MaxVolume * VolDimScale;
		Range.Min = Range.Min * Range.Min * Range.Min;
		Range.Max = Range.Max * Range.Max * Range.Max;
	}
	else if (SelectionSettings->VolumeSelectionMethod == EVolumeSelectionMethod::RelativeToLargest)
	{
		double LargestVolume = KINDA_SMALL_NUMBER;
		for (int32 Idx = 0; Idx < Volumes.Num(); Idx++)
		{
			if (SimulationType[Idx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				float Volume = Volumes[Idx];
				LargestVolume = FMath::Max(LargestVolume, Volume);
			}
		}
		Range.Min = LargestVolume * SelectionSettings->MinVolumeFrac * SelectionSettings->MinVolumeFrac * SelectionSettings->MinVolumeFrac;
		Range.Max = LargestVolume * SelectionSettings->MaxVolumeFrac * SelectionSettings->MaxVolumeFrac * SelectionSettings->MaxVolumeFrac;
	}
	else // EVolumeSelectionMethod::RelativeToWhole
	{
		double VolumeSum = 0;
		for (int32 Idx = 0; Idx < Volumes.Num(); Idx++)
		{
			if (SimulationType[Idx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				float Volume = Volumes[Idx];
				VolumeSum += Volume;
			}
		}
		Range.Min = VolumeSum * SelectionSettings->MinVolumeFrac * SelectionSettings->MinVolumeFrac * SelectionSettings->MinVolumeFrac;
		Range.Max = VolumeSum * SelectionSettings->MaxVolumeFrac * SelectionSettings->MaxVolumeFrac * SelectionSettings->MaxVolumeFrac;
	}
	return Range;
}

bool UFractureToolSelection::GetBonesByVolume(const FGeometryCollection& Collection, TArray<int32>& FilterIndices)
{
	const TManagedArray<float>* Volumes = Collection.FindAttribute<float>("Volume", FGeometryCollection::TransformGroup);
	if (!Volumes)
	{
		UE_LOGF(LogFractureTool, Warning, "Volumes missing from geometry collection; could not filter selection by volume.");
		return false;
	}

	const UFractureSettings* FractureSettings = GetDefault<UFractureSettings>();

	TInterval<float> VolumeRange = GetVolumeRange(*Volumes, Collection.SimulationType);

	const TManagedArray<int32>* Levels = Collection.FindAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& SimTypes = Collection.SimulationType;

	for (int32 BoneIdx = 0; BoneIdx < Volumes->Num(); BoneIdx++)
	{
		// Skip embedded geometry (volumes aren't computed for embedded geo)
		if (SimTypes[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			continue;
		}
		// only consider bones at the filtered level (if any)
		if (FractureSettings->FractureLevel > -1 && Levels && (*Levels)[BoneIdx] != FractureSettings->FractureLevel)
		{
			continue;
		}
		if (VolumeRange.Contains((*Volumes)[BoneIdx]))
		{
			FilterIndices.Add(BoneIdx);
		}
	}

	int TargetNumBones = FMath::Max(0, FilterIndices.Num() * SelectionSettings->KeepFraction);
	if (TargetNumBones < FilterIndices.Num())
	{
		FRandomStream Random(SelectionSettings->RandomSeed);

		TArray<FBox> AllBounds = GetAllBounds(Collection);
		TArray<FVector> Points;
		TArray<float> Weights;
		Points.Reserve(FilterIndices.Num());
		Weights.Reserve(FilterIndices.Num());
		for (int Idx : FilterIndices)
		{
			FVector Point = AllBounds[Idx].GetCenter();
			Points.Add(Point);
			// set random importance weights, so the point ordering will be shuffled
			Weights.Add((float)Random.RandHelper(FilterIndices.Num()) / FilterIndices.Num());
		}
		UE::Geometry::FPriorityOrderPoints Ordering;
		Ordering.ComputeUniformSpaced(Points, Weights, TargetNumBones, 1);
		TArray<int32> ToKeep;
		ToKeep.Reserve(TargetNumBones);
		for (int32 OrderIdx = 0; OrderIdx < TargetNumBones; OrderIdx++)
		{
			ToKeep.Add(FilterIndices[Ordering.Order[OrderIdx]]);
		}
		FilterIndices = ToKeep;
	}

	return true;
}

void UFractureToolSelection::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = InToolkit.Pin().Get();

		TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

		for (FFractureToolContext& FractureContext : FractureContexts)
		{
			FractureContext.GetFracturedGeometryCollection()->Modify();
			FractureContext.GetGeometryCollectionComponent()->Modify();

			UpdateSelection(FractureContext);

			Refresh(FractureContext, Toolkit);
		}

		SetOutlinerComponents(FractureContexts, Toolkit);
	}
}

void UFractureToolSelection::UpdateSelection(FFractureToolContext& FractureContext)
{
	if (!FractureContext.GetGeometryCollection().IsValid())
	{
		return;
	}

	FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

	TArray<int32> FilterIndices;
	if (!GetBonesByVolume(Collection, FilterIndices))
	{
		return;
	}

	UFractureEditorMode* FractureMode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (FractureMode)
	{
		bool bAppend = SelectionSettings->SelectionOperation == ESelectionOperation::Add, bRemove = SelectionSettings->SelectionOperation == ESelectionOperation::Remove;
		FractureMode->UpdateSelection(FractureContext.GetGeometryCollectionComponent()->GetSelectedBones(), FilterIndices, bAppend, bRemove);
		FractureContext.SetSelection(FilterIndices);
	}
}


#undef LOCTEXT_NAMESPACE
