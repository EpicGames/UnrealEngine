// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"
#include "Engine/StaticMeshActor.h"


#include "Mechanics/RectangleMarqueeMechanic.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "FractureToolSelection.generated.h"

class FFractureToolContext;

UENUM()
enum class EVolumeSelectionMethod
{
	// Select by cube root of volume
	CubeRootOfVolume,
	// Select by cube root of volume relative to the overall shape's cube root of volume
	RelativeToWhole,
	// Select by cube root of volume relative to the largest single geometry
	RelativeToLargest
};

UENUM()
enum class ESelectionOperation
{
	Replace,
	Add,
	Remove
};

UENUM()
enum class EMouseSelectionMethod
{
	// Click+Drag a rectangle to select -- hold shift to append, ctrl to subtract, and ctrl+shift to toggle
	RectSelect,
	// Selection that works the same as when the tool is not active
	StandardSelect
	// TODO: lasso selection, brush selection?
};

/** Settings controlling how geometry is selected */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureSelectionSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureSelectionSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	UPROPERTY(EditAnywhere, Category = MouseInteraction)
	EMouseSelectionMethod MouseSelectionMethod = EMouseSelectionMethod::RectSelect;

	/** What values to use when filtering by volume.  Note all values are presented as cube roots to give more intuitive scales (e.g., to select bones with volume less than a 10x10x10 cube, choose CubeRootOfVolume and MaxVolume=10, rather than needing to multiply out to 1000) */
	UPROPERTY(EditAnywhere, Category = FilterSettings)
	EVolumeSelectionMethod VolumeSelectionMethod = EVolumeSelectionMethod::RelativeToLargest;

	/** How to update the selection from the filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings)
	ESelectionOperation SelectionOperation = ESelectionOperation::Replace;

	/** Sets the minimum volume (as computed by the Volume Selection Method) that should be included in the filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = ".00001", UIMin = ".1", UIMax = "1000", EditCondition = "VolumeSelectionMethod == EVolumeSelectionMethod::CubeRoot", EditConditionHides))
	double MinVolume = 0;

	/** Sets the maximum volume (as computed by the Volume Selection Method) that should be included in the filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = ".00001", UIMin = ".1", UIMax = "1000", EditCondition = "VolumeSelectionMethod == EVolumeSelectionMethod::CubeRoot", EditConditionHides))
	double MaxVolume = 0;

	/** Sets the minimum volume (as computed by the Volume Selection Method) that should be included in the filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = ".00001", UIMin = ".01", UIMax = "1", EditCondition = "VolumeSelectionMethod != EVolumeSelectionMethod::CubeRoot", EditConditionHides))
	double MinVolumeFrac = 0;

	/** Sets the maximum volume (as computed by the Volume Selection Method) that should be included in the filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = ".00001", UIMin = ".01", UIMax = "1", EditCondition = "VolumeSelectionMethod != EVolumeSelectionMethod::CubeRoot", EditConditionHides))
	double MaxVolumeFrac = 0;

	/** Fraction of bones to keep in the selection: If less than 1, bones will be randomly excluded from the selection filter */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = ".00001", UIMin = ".01", UIMax = "1"))
	double KeepFraction = 1;

	/** Seed to use for randomization when deciding which bones to keep w/ the Keep Fraction*/
	UPROPERTY(EditAnywhere, Category = FilterSettings)
	int RandomSeed = 1;
};




/*
 * Adapted from URectangleMarqueeMechanic, the mechanic for a rectangle "marquee" selection, to operate without
 * the UInteractionMechanic framework. It creates and maintains the 2D rectangle associated with a mouse drag.
 * It does not test against any scene geometry, nor does it maintain any sort of list of selected objects.
 *
 * You must manually call the Setup(), Render() and DrawHUD() functions.
 * Note these functions have been changed from the Mechanic versions to be easier to call without additional InteractiveTool machinery
 *
 * Attach to the delegates and use the passed rectangle to test against your geometry.
 * 
 * TODO: Refactor this into a more generalized / generally usable class in RectangleMarqueeMechanic.h,
 * and share more common code with URectangleMarqueeMechanic.
 */
UCLASS()
class URectangleMarqueeManager : public UObject, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * If true, then the URectangleMarqueeManager will not create an internal UClickDragInputBehavior in ::Setup(), allowing
	 * the client to control the marquee with an external InputBehavior that uses the marquee manager as its IClickDragBehaviorTarget.
	 * For instance, this allows the mechanic to be hooked in as the drag component of a USingleClickOrDragInputBehavior.
	 * Must be configured before calling ::Setup()
	 */
	UPROPERTY()
	bool bUseExternalClickDragBehavior = false;

	/**
	 * If the computation time for a single call to OnDragRectangleChanged ever exceeds this threshold then future calls
	 * to this function (in the current drag sequence) will be deferred until the mouse button is released. This will
	 * improve the responsiveness of the UI. The default value is set so this optimization is never triggered, if you
	 * want it you can set this to a small value e.g., 1./60 (time elapsed by 1 frame at 60 fps)
	 */
	UPROPERTY()
	double OnDragRectangleChangedDeferredThreshold = TNumericLimits<double>::Max(); // In seconds

public:

	void Setup(TFunctionRef<void(UInputBehavior*, void*)> AddBehaviorFunc);
	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);
	void DrawHUD(FCanvas* Canvas, bool bThisViewHasFocus);

	bool IsEnabled();
	void SetIsEnabled(bool bOn);

	/**
	 * Sets the base priority so that users can make sure that their own behaviors are higher
	 * priority. The marquee will not use any priority value higher than this.
	 *
	 * Can be called before or after Setup(). Only relevant if bUseExternalClickDragBehavior is false
	 */
	void SetBasePriority(const FInputCapturePriority& Priority);

	/**
	 * Gets the current priority used by the marquee behavior
	 */
	FInputCapturePriority GetPriority() const;

	/**
	 * Called when user starts dragging a new rectangle.
	 */
	FSimpleMulticastDelegate OnDragRectangleStarted;

	/**
	 * Called as the user drags the other corner of the rectangle around.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnDragRectangleChangedEvent, const FCameraRectangle&);
	OnDragRectangleChangedEvent OnDragRectangleChanged;

	/**
	 * Called once the user lets go of the mouse button after dragging out a rectangle.
	 * The last dragged rectangle is passed here so that clients can choose to just implement this function in simple cases.
	 * bCancelled flag is true when the drag finishes due to a disabling of the mechanic or due to a TerminateDragSequence call, rather than a normal drag completion.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(OnDragRectangleFinishedEvent, const FCameraRectangle&, bool bCancelled);
	OnDragRectangleFinishedEvent OnDragRectangleFinished;

protected:
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior = nullptr;

	FCameraRectangle CameraRectangle;

	FInputCapturePriority BasePriority = FInputCapturePriority(FInputCapturePriority::DEFAULT_TOOL_PRIORITY);

private:

	FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos);
	void OnClickPress(const FInputDeviceRay& PressPos);
	void OnClickDrag(const FInputDeviceRay& DragPos);
	void OnClickRelease(const FInputDeviceRay& ReleasePos);
	void OnTerminateDragSequence();

	bool bIsEnabled;
	bool bIsDragging;
	bool bIsOnDragRectangleChangedDeferred;
};




// Note this tool doesn't actually fracture, but it does remake pieces of geometry and shares a lot of machinery with the fracture tools
UCLASS(DisplayName = "Custom Selection Tool", Category = "FractureTools")
class UFractureToolSelection : public UFractureToolCutterBase, public IClickBehaviorTarget
{
public:
	GENERATED_BODY()

	UFractureToolSelection(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureSelection", "ExecuteSelection", "Update Selection")); }

	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;

	virtual TArray<UObject*> GetSettingsObjects() const override;

	virtual void FractureContextChanged() override;
	// ExecuteFracture unused, because this tool doesn't modify geometry; instead, it overrides Execute
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override { return INDEX_NONE; }
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	void UpdateSelection(FFractureToolContext& FractureContext);

	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual void Shutdown() override;

	// IClickBehaviorTarget implementation
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override
	{
		return FInputRayHit(FMathf::MaxReal); // Always hit (with MaxReal to lose tiebreakers); actually hit testing done later (in OnClicked)
	}
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	void CreateRectangleSelectionBehavior();
	void DestroyRectangleSelectionBehavior();

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		SelectionBounds.Empty();
		SelectionMappings.Empty();
	}

	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> SelectionBehaviorSet = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalInputBehaviorSource> SelectionBehaviorSource = nullptr;

	UPROPERTY()
	TObjectPtr<URectangleMarqueeManager> RectangleMarqueeManager = nullptr;

	// the tools context responsible for the selection input router
	UPROPERTY()
	TObjectPtr<UInteractiveToolsContext> UsedToolsContext = nullptr;

	// Support for Shift and Ctrl toggle
	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	// Rectangle selection
	void OnDragRectangleStarted();
	void OnDragRectangleChanged(const FCameraRectangle& Rectangle);
	void OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

private:

	UPROPERTY(EditAnywhere, Category = FixGeo)
	TObjectPtr<UFractureSelectionSettings> SelectionSettings;

	TArray<FBox> SelectionBounds; // Bounds in global space but without exploded vectors applied
	FVisualizationMappings SelectionMappings;

	TInterval<float> GetVolumeRange(const TManagedArray<float>& Volumes, const TManagedArray<int32>& SimulationType) const;
	bool GetBonesByVolume(const FGeometryCollection& Collection, TArray<int32>& FilterIndices);
	const double VolDimScale = .01; // compute volumes in meters instead of cm, for saner units at typical scales

};


