// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolSelectors.h"

#include "Editor.h"
#include "Engine/Selection.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollection.h"
#include "FractureSelectionTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolSelectors)


#define LOCTEXT_NAMESPACE "FractureToolSelectionOps"


FText UFractureToolSelectAll::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectAll", "Select All"));
}

FText UFractureToolSelectAll::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectAllTooltip", "Selects all bones in the Geometry Collection"));
}

FSlateIcon UFractureToolSelectAll::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectAll");
}

void UFractureToolSelectAll::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectAll", "All", "Selects all bones in the Geometry Collection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::A));
	BindingContext->SelectAll = UICommandInfo;
}

void UFractureToolSelectAll::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::AllGeometry);
	}
}

void UFractureToolSelectAll::SelectByMode(FFractureEditorModeToolkit* InToolkit, GeometryCollection::ESelectionMode SelectionMode)
{
	USelection* SelectionSet = GEditor->GetSelectedActors();

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());
	SelectionSet->GetSelectedObjects(SelectedActors);

	for (AActor* Actor : SelectedActors)
	{
		TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
		Actor->GetComponents(GeometryCollectionComponents);

		for (UGeometryCollectionComponent* GeometryCollectionComponent : GeometryCollectionComponents)
		{
			FScopedColorEdit EditBoneColor = GeometryCollectionComponent->EditBoneSelection();
			EditBoneColor.SelectBones(SelectionMode);

			// Increase level so that we can see newly selected pieces
			int32 LevelView = InToolkit->GetLevelViewValue();
			if (LevelView >= 0)
			{
				int32 TargetLevel = LevelView;
				// if we've selected bones beyond the current level, try to increase the level
				int32 MaxLevel = EditBoneColor.GetMaxSelectedLevel(false);
				if (MaxLevel > LevelView)
				{
					TargetLevel = MaxLevel;
				}
				// if the selected bones would not show up in the outliner at the target level, instead choose "All" level
				if (!EditBoneColor.IsSelectionValidAtLevel(TargetLevel))
				{
					TargetLevel = -1;
				}
				if (TargetLevel != LevelView)
				{
					EditBoneColor.SetLevelViewMode(TargetLevel);
					InToolkit->OnSetLevelViewValue(TargetLevel);
				}
			}

			InToolkit->SetBoneSelection(GeometryCollectionComponent, EditBoneColor.GetSelectedBones(), true);
		}
	}
}



FText UFractureToolSelectNone::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectNone", "Select None"));
}

FText UFractureToolSelectNone::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectNoneTooltip", "Deselects all bones in the Geometry Collection"));
}

FSlateIcon UFractureToolSelectNone::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectNone");
}

void UFractureToolSelectNone::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectNone", "None", "Deselects all bones in the Geometry Collection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::D));
	BindingContext->SelectNone = UICommandInfo;
}

void UFractureToolSelectNone::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::None);
	}
}


FText UFractureToolSelectNeighbors::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectNeighbors", "Select Neighbors"));
}

FText UFractureToolSelectNeighbors::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectNeighborsTooltip", "Select all bones adjacent to the currently selected bones."));
}

FSlateIcon UFractureToolSelectNeighbors::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectNeighbors");
}

void UFractureToolSelectNeighbors::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectNeighbors", "Contact", "Select all bones adjacent to the currently selected bones.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectNeighbors = UICommandInfo;
}

void UFractureToolSelectNeighbors::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Neighbors);
	}
}


FText UFractureToolSelectParent::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectParent", "Select Parent"));
}

FText UFractureToolSelectParent::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectParentTooltip", "Select clusters containing the currently selected bones."));
}

FSlateIcon UFractureToolSelectParent::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectParent");
}

void UFractureToolSelectParent::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectParent", "Parent", "Select clusters containing the currently selected bones.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectParent = UICommandInfo;
}

void UFractureToolSelectParent::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Parent);
	}
}



FText UFractureToolSelectChildren::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectChildren", "Select Children"));
}

FText UFractureToolSelectChildren::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectChildrenTooltip", "Select all bones that are immediate children of the currently selected clusters."));
}

FSlateIcon UFractureToolSelectChildren::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectChildren");
}

void UFractureToolSelectChildren::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectChildren", "Children", "Select all bones that are immediate children of the currently selected clusters.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectChildren = UICommandInfo;
}

void UFractureToolSelectChildren::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Children);
	}
}




FText UFractureToolSelectSiblings::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectSiblings", "Select Siblings"));
}

FText UFractureToolSelectSiblings::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectSiblingsTooltip", "Select all bones sharing the cluster with currently selected bones."));
}

FSlateIcon UFractureToolSelectSiblings::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectSiblings");
}

void UFractureToolSelectSiblings::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectSiblings", "Siblings", "Select all bones sharing the cluster with currently selected bones.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectSiblings = UICommandInfo;
}

void UFractureToolSelectSiblings::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Siblings);
	}
}


FText UFractureToolSelectAllInLevel::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectAllInLevel", "Select All In Level"));
}

FText UFractureToolSelectAllInLevel::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectAllInLevelTooltip", "Select all bones at the same level as currently selected bones."));
}

FSlateIcon UFractureToolSelectAllInLevel::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectAllInLevel");
}

void UFractureToolSelectAllInLevel::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectAllInLevel", "Level", "Select all bones at the same level as currently selected bones.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectAllInLevel = UICommandInfo;
}

void UFractureToolSelectAllInLevel::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Level);
	}
}



FText UFractureToolSelectInvert::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectInvert", "Invert Selection"));
}

FText UFractureToolSelectInvert::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectInvertTooltip", "Invert current selection of bones."));
}

FSlateIcon UFractureToolSelectInvert::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectInvert");
}

void UFractureToolSelectInvert::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectInvert", "Invert", "Invert current selection of bones.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectInvert = UICommandInfo;
}

void UFractureToolSelectInvert::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::InverseGeometry);
	}
}




FText UFractureToolSelectLeaf::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectLeaf", "Select Leaves"));
}

FText UFractureToolSelectLeaf::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectLeafTooltip", "Select leaf bones that represent rigid bodies at current level"));
}

FSlateIcon UFractureToolSelectLeaf::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectLeaf");
}

void UFractureToolSelectLeaf::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectLeaf", "Leaf", "Select (rigid) leaf bones at current level.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectLeaves = UICommandInfo;
}

void UFractureToolSelectLeaf::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Leaves);
	}
}




FText UFractureToolSelectCluster::GetDisplayText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectCluster", "Select Clusters"));
}

FText UFractureToolSelectCluster::GetTooltipText() const
{
	return FText(NSLOCTEXT("Fracture", "FractureToolSelectClusterTooltip", "Select cluster bones at current level"));
}

FSlateIcon UFractureToolSelectCluster::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SelectCluster");
}

void UFractureToolSelectCluster::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SelectCluster", "Cluster", "Select cluster bones at current level.", EUserInterfaceActionType::Button, FInputChord());
	BindingContext->SelectClusters = UICommandInfo;
}

void UFractureToolSelectCluster::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	if (InToolkit.IsValid())
	{
		SelectByMode(InToolkit.Pin().Get(), GeometryCollection::ESelectionMode::Clusters);
	}
}

#undef LOCTEXT_NAMESPACE
