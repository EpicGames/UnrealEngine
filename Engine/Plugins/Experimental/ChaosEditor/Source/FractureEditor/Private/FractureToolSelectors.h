// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolSelectors.generated.h"


namespace GeometryCollection
{
	enum class ESelectionMode : uint8;
}

class FFractureEditorModeToolkit;


UCLASS(DisplayName = "Select All", Category = "FractureTools")
class UFractureToolSelectAll : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureToolSelectAll(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;

protected:
	void SelectByMode(FFractureEditorModeToolkit* InToolkit, GeometryCollection::ESelectionMode SelectionMode);
};


UCLASS(DisplayName = "Select None", Category = "FractureTools")
class UFractureToolSelectNone : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectNone(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Neighbors", Category = "FractureTools")
class UFractureToolSelectNeighbors : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectNeighbors(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Parent", Category = "FractureTools")
class UFractureToolSelectParent : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectParent(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Children", Category = "FractureTools")
class UFractureToolSelectChildren : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectChildren(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};

UCLASS(DisplayName = "Select Siblings", Category = "FractureTools")
class UFractureToolSelectSiblings : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectSiblings(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select All In Level", Category = "FractureTools")
class UFractureToolSelectAllInLevel : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectAllInLevel(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Invert", Category = "FractureTools")
class UFractureToolSelectInvert : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectInvert(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Leaves", Category = "FractureTools")
class UFractureToolSelectLeaf : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectLeaf(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};


UCLASS(DisplayName = "Select Clusters", Category = "FractureTools")
class UFractureToolSelectCluster : public UFractureToolSelectAll
{
public:
	GENERATED_BODY()

	UFractureToolSelectCluster(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// UFractureActionTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
};
