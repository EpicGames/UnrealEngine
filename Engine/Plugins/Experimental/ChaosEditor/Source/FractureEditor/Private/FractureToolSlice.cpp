// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolSlice.h"

#include "Editor.h"
#include "Engine/Selection.h"
#include "FractureTool.h"
#include "FractureEditorStyle.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "PlanarCut.h"
#include "FractureEditorModeToolkit.h"
#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolSlice)

#define LOCTEXT_NAMESPACE "FractureSlice"



UFractureToolSlice::UFractureToolSlice(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	SliceSettings = NewObject<UFractureSliceSettings>(GetTransientPackage(), UFractureSliceSettings::StaticClass());
	SliceSettings->OwnerTool = this;
}

FText UFractureToolSlice::GetDisplayText() const
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolSlice", "Slice Fracture")); 
}

FText UFractureToolSlice::GetTooltipText() const 
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolSliceTooltip", "The Slice fracture method enables you to define the number of X, Y, and Z slices, along with providing random angle and offset variation.  Click the Fracture Button to commit the fracture to the geometry collection.")); 
}

FSlateIcon UFractureToolSlice::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Slice");
}

void UFractureToolSlice::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Slice", "Slice", "Fracture with a grid of X, Y, and Z slices, with optional random variation in angle and offset.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Slice = UICommandInfo;
}

TArray<UObject*> UFractureToolSlice::GetSettingsObjects() const 
{ 
	TArray<UObject*> Settings; 
	Settings.Add(SliceSettings);
	Settings.Add(CutterSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

void UFractureToolSlice::GenerateSliceTransforms(const FFractureToolContext& Context, TArray<FTransform>& CuttingPlaneTransforms)
{
	const FBox& Bounds = Context.GetWorldBounds();
	const FVector& Min = Bounds.Min;
	const FVector& Max = Bounds.Max;
	const FVector  Center = Bounds.GetCenter();
	const FVector Extents(Max - Min);
	const FVector HalfExtents(Extents * 0.5f);

	const FVector Step(Extents.X / (SliceSettings->SlicesX+1), Extents.Y / (SliceSettings->SlicesY+1), Extents.Z / (SliceSettings->SlicesZ+1));

	CuttingPlaneTransforms.Reserve(SliceSettings->SlicesX * SliceSettings->SlicesY * SliceSettings->SlicesZ);

	FRandomStream RandomStream(Context.GetSeed());

	const float SliceAngleVariationInRadians = FMath::DegreesToRadians(SliceSettings->SliceAngleVariation);

	const FVector XMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector XMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 xx = 0; xx < SliceSettings->SlicesX; ++xx)
	{
		const FVector SlicePosition(FVector(Min.X, Center.Y, Center.Z) + FVector((Step.X * xx) + Step.X, 0.0f, 0.0f) + RandomStream.VRand() * RandomStream.GetFraction() * SliceSettings->SliceOffsetVariation);
		FTransform Transform(FQuat(FVector::RightVector, FMath::DegreesToRadians(90)), SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(FQuat(RotA * RotB));
		CuttingPlaneTransforms.Emplace(Transform);
	}

	const FVector YMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector YMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 yy = 0; yy < SliceSettings->SlicesY; ++yy)
	{
		const FVector SlicePosition(FVector(Center.X, Min.Y, Center.Z) + FVector(0.0f, (Step.Y * yy) + Step.Y, 0.0f) + RandomStream.VRand() * RandomStream.GetFraction() * SliceSettings->SliceOffsetVariation);
		FTransform Transform(FQuat(FVector::ForwardVector, FMath::DegreesToRadians(90)), SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(RotA * RotB);
		CuttingPlaneTransforms.Emplace(Transform);
	}

	const FVector ZMin(-Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));
	const FVector ZMax(Bounds.GetExtent() * FVector(1.0f, 1.0f, 0.0f));

	for (int32 zz = 0; zz < SliceSettings->SlicesZ; ++zz)
	{
		const FVector SlicePosition(FVector(Center.X, Center.Y, Min.Z) + FVector(0.0f, 0.0f, (Step.Z * zz) + Step.Z) + RandomStream.VRand() * RandomStream.GetFraction() * SliceSettings->SliceOffsetVariation);
		FTransform Transform(SlicePosition);
		const FQuat RotA(FVector::RightVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		const FQuat RotB(FVector::ForwardVector, RandomStream.FRandRange(0.0f, SliceAngleVariationInRadians));
		Transform.ConcatenateRotation(RotA * RotB);
		CuttingPlaneTransforms.Emplace(Transform);
	}
}

void UFractureToolSlice::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (CutterSettings->bDrawDiagram)
	{
		EnumerateVisualizationMapping(PlanesMappings, RenderCuttingPlanesTransforms.Num(), [&](int32 Idx, FVector ExplodedVector)
		{
			const FTransform& Transform = RenderCuttingPlanesTransforms[Idx];
			FVector Center = Transform.GetLocation() + ExplodedVector;
			PDI->DrawPoint(Center, FLinearColor::Green, 4.f, SDPG_Foreground);

			PDI->DrawLine(Center, Center + Transform.GetUnitAxis(EAxis::X) * RenderCuttingPlaneSize, FLinearColor(255, 0, 0), SDPG_Foreground);
			PDI->DrawLine(Center, Center + Transform.GetUnitAxis(EAxis::Y) * RenderCuttingPlaneSize, FLinearColor(0, 255, 0), SDPG_Foreground);

			PDI->DrawLine(Center + Transform.GetUnitAxis(EAxis::X) * RenderCuttingPlaneSize, Center + Transform.GetUnitAxis(EAxis::X) * RenderCuttingPlaneSize + Transform.GetUnitAxis(EAxis::Y) * RenderCuttingPlaneSize, FLinearColor(255, 0, 0), SDPG_Foreground);
			PDI->DrawLine(Center + Transform.GetUnitAxis(EAxis::Y) * RenderCuttingPlaneSize, Center + Transform.GetUnitAxis(EAxis::X) * RenderCuttingPlaneSize + Transform.GetUnitAxis(EAxis::Y) * RenderCuttingPlaneSize, FLinearColor(0, 255, 0), SDPG_Foreground);
		});
	}
}

void UFractureToolSlice::FractureContextChanged()
{
	UpdateDefaultRandomSeed();
	TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

	ClearVisualizations();

	RenderCuttingPlaneSize = FLT_MAX;
	for (FFractureToolContext& FractureContext : FractureContexts)
	{
		FBox Bounds = FractureContext.GetWorldBounds();
		if (!Bounds.IsValid)
		{
			continue;
		}
		int32 CollectionIdx = VisualizedCollections.Add(FractureContext.GetGeometryCollectionComponent());
		int32 BoneIdx = FractureContext.GetSelection().Num() == 1 ? FractureContext.GetSelection()[0] : INDEX_NONE;
		PlanesMappings.AddMapping(CollectionIdx, BoneIdx, RenderCuttingPlanesTransforms.Num());

		GenerateSliceTransforms(FractureContext, RenderCuttingPlanesTransforms);

		if (Bounds.GetExtent().GetMax() < RenderCuttingPlaneSize)
		{
			RenderCuttingPlaneSize = Bounds.GetExtent().GetMax();
		}
	}
}

int32 UFractureToolSlice::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if(FractureContext.IsValid())
	{
		TArray<FTransform> LocalCuttingPlanesTransforms;
		GenerateSliceTransforms(FractureContext, LocalCuttingPlanesTransforms);

		TArray<FPlane> CuttingPlanes;
		CuttingPlanes.Reserve(LocalCuttingPlanesTransforms.Num());

		for (const FTransform& Transform : LocalCuttingPlanesTransforms)
		{
			CuttingPlanes.Add(FPlane(Transform.GetLocation(), Transform.GetUnitAxis(EAxis::Z)));
		}

		FInternalSurfaceMaterials InternalSurfaceMaterials;
		FNoiseSettings NoiseSettings;
		if (CutterSettings->Amplitude > 0.0f)
		{
			CutterSettings->TransferNoiseSettings(NoiseSettings);
			InternalSurfaceMaterials.NoiseSettings = NoiseSettings;
		}

		// Proximity is invalidated.
		ClearProximity(FractureContext.GetGeometryCollection().Get());

		return CutMultipleWithMultiplePlanes(CuttingPlanes, InternalSurfaceMaterials, *FractureContext.GetGeometryCollection(), FractureContext.GetSelection(), CutterSettings->Grout, CollisionSettings->GetPointSpacing(), FractureContext.GetSeed(), FractureContext.GetTransform(), true, nullptr, FIslandSplitSettings(CutterSettings->bSplitIslands, CutterSettings->CloseVertexDistance, CutterSettings->VertexToSurfaceBridgeDistance));
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE
