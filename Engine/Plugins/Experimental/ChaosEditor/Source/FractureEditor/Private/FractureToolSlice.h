// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"

#include "FractureToolSlice.generated.h"


class FFractureToolContext;


UCLASS(config = EditorPerProjectUserSettings)
class UFractureSliceSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureSliceSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, SlicesX(3)
		, SlicesY(3)
		, SlicesZ(1)
		, SliceAngleVariation(0.0f)
		, SliceOffsetVariation(0.0f)
	{}

	/** Number of slices along the X axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (UIMin = "0"))
	int32 SlicesX;

	/** Number of slices along the Y axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (UIMin = "0"))
	int32 SlicesY;

	/** Number of slices along the Z axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (UIMin = "0"))
	int32 SlicesZ;

	/** Maximum angle (in degrees) to randomly rotate each slicing plane */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DisplayName = "Random Angle Variation", UIMin = "0.0", UIMax = "90.0"))
	float SliceAngleVariation;

	/** Maximum distance (in cm) to randomly shift each slicing plane */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DisplayName = "Random Offset Variation", UIMin = "0.0"))
	float SliceOffsetVariation;
};

UCLASS(DisplayName="Slice Tool", Category="FractureTools")
class UFractureToolSlice : public UFractureToolCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolSlice(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;

	virtual void RegisterUICommand( FFractureEditorCommands* BindingContext ) override;
	
	virtual TArray<UObject*> GetSettingsObjects() const override;
	void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	virtual void FractureContextChanged() override;
	virtual int32 ExecuteFracture(const FFractureToolContext& Context) override;

	void GenerateSliceTransforms(const FFractureToolContext& Context, TArray<FTransform>& CuttingPlaneTransforms);

	virtual void ConfigureCutterSettings() override
	{
		CutterSettings->bGroupFractureToggleEnabled = true;
		CutterSettings->bDrawSitesToggleEnabled = false;
		CutterSettings->bNoisePreviewToggleEnabled = false; // TODO: support noise preview for slices
	}

	// Slicing
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureSliceSettings> SliceSettings;

protected:
	virtual void ClearVisualizations() override
	{
		Super::ClearVisualizations();
		RenderCuttingPlanesTransforms.Empty();
		PlanesMappings.Empty();
	}

private:
	float RenderCuttingPlaneSize;
	TArray<FTransform> RenderCuttingPlanesTransforms;
	FVisualizationMappings PlanesMappings;
};