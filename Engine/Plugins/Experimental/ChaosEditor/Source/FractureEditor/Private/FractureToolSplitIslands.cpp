// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolSplitIslands.h"

#include "FractureToolCutter.h"
#include "FractureToolContext.h"
#include "PlanarCut.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolSplitIslands)

#define LOCTEXT_NAMESPACE "FractureToolSplitIslands"

UFractureToolSplitIslands::UFractureToolSplitIslands(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SplitIslandsSettings = NewObject<UFractureSplitIslandsSettings>(GetTransientPackage(), UFractureSplitIslandsSettings::StaticClass());
	SplitIslandsSettings->OwnerTool = this;
	CollisionSettings = NewObject<UFractureCollisionSettings>(GetTransientPackage(), UFractureCollisionSettings::StaticClass());
	CollisionSettings->OwnerTool = this;
}

bool UFractureToolSplitIslands::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolSplitIslands::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolSplitIslands", "Split Islands"));
}

FText UFractureToolSplitIslands::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolSplitIslandsTooltip", "Split selected geometry into islands based on mesh connectivity and distances"));
}

FSlateIcon UFractureToolSplitIslands::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.SplitIslands");
}

void UFractureToolSplitIslands::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "SplitIslands", "Split", "Split selected geometry into islands based on mesh connectivity and distances", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->SplitIslands = UICommandInfo;
}

TArray<UObject*> UFractureToolSplitIslands::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(SplitIslandsSettings);
	Settings.Add(CollisionSettings);
	return Settings;
}

TArray<FFractureToolContext> UFractureToolSplitIslands::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			FFractureToolContext Context(GeometryCollectionComponent);
			Context.Sanitize();
			Context.ConvertSelectionToLeafNodes();
			Contexts.Add(Context);
		}
	}

	return Contexts;
}

int32 UFractureToolSplitIslands::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	FGeometryCollection& Collection = *FractureContext.GetGeometryCollection();

	FIslandSplitSettings Settings;
	Settings.CloseVertexDistance = SplitIslandsSettings->CloseVertexDistance;
	Settings.VertexToSurfaceBridgeDistance = SplitIslandsSettings->VertexToSurfaceBridgeDistance;

	return ::SplitIslands(
		Collection,
		FractureContext.GetSelection(),
		CollisionSettings->GetPointSpacing(),
		nullptr, // Progress
		Settings);
}

#undef LOCTEXT_NAMESPACE
