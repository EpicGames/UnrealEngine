// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"

#include "FractureToolSplitIslands.generated.h"

class FFractureToolContext;
class UFractureCollisionSettings;

/** Settings for the Split Islands tool */
UCLASS(config = EditorPerProjectUserSettings)
class UFractureSplitIslandsSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureSplitIslandsSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{}

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Split Islands", meta = (Units = "cm", ClampMin = "0.0"))
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Split Islands", meta = (Units = "cm", ClampMin = "0.0"))
	float VertexToSurfaceBridgeDistance = 0.f;

};


UCLASS(DisplayName = "Split Islands", Category = "FractureTools")
class UFractureToolSplitIslands : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolSplitIslands(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("FractureSplitIslands", "ExecuteSplitIslands", "Split Islands")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	UPROPERTY(EditAnywhere, Category = SplitIslands)
	TObjectPtr<UFractureSplitIslandsSettings> SplitIslandsSettings;

	UPROPERTY(EditAnywhere, Category = Collision)
	TObjectPtr<UFractureCollisionSettings> CollisionSettings;

};
