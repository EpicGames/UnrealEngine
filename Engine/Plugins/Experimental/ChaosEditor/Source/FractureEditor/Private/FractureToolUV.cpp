// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolUV.h"

#include "FractureToolContext.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "AssetUtils/Texture2DBuilder.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorAssetLibrary.h"
// for content-browser things
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"

#include "FractureToolBackgroundTask.h"

#include "FractureAutoUV.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolUV)

#define LOCTEXT_NAMESPACE "FractureToolAutoUV"

using namespace UE::Geometry;
using namespace UE::Fracture;


namespace
{
	UE::PlanarCut::ETargetFaces ConvertTargetFacesEnum(ETargetFaces TargetFaces)
	{
		return
			TargetFaces == ETargetFaces::SelectedMaterialIDs ? UE::PlanarCut::ETargetFaces::CustomFaces :
			TargetFaces == ETargetFaces::AllFaces ? UE::PlanarCut::ETargetFaces::AllFaces :
			TargetFaces == ETargetFaces::ExternalFaces ? UE::PlanarCut::ETargetFaces::ExternalFaces :
			UE::PlanarCut::ETargetFaces::InternalFaces;
	}

	bool TargetsMaterialIDs(ETargetFaces TargetFaces)
	{
		return TargetFaces != ETargetFaces::InternalFaces && TargetFaces != ETargetFaces::ExternalFaces && TargetFaces != ETargetFaces::AllFaces;
	}
}


void UFractureAutoUVSettings::SetNumUVChannels(int32 NumUVChannels)
{
	NumUVChannels = FMath::Clamp<int32>(NumUVChannels, 1, GeometryCollectionUV::MAX_NUM_UV_CHANNELS);
	UVChannelNamesList.Reset();
	for (int32 k = 0; k < NumUVChannels; ++k)
	{
		UVChannelNamesList.Add(FString::Printf(TEXT("UV %d"), k));
	}
	if (GetSelectedChannelIndex(false) == INDEX_NONE)
	{
		UVChannel = UVChannelNamesList[0];
	}
}

int32 UFractureAutoUVSettings::GetSelectedChannelIndex(bool bForceToZeroOnFailure)
{
	int32 FoundIndex = UVChannelNamesList.IndexOfByKey(UVChannel);
	if (FoundIndex == INDEX_NONE && bForceToZeroOnFailure)
	{
		FoundIndex = 0;
	}
	return FoundIndex;
}

void UFractureAutoUVSettings::ChangeNumUVChannels(int32 Delta)
{
	int32 Target = UVChannelNamesList.Num() + Delta;
	if (Target > 0 && Target <= GeometryCollectionUV::MAX_NUM_UV_CHANNELS)
	{
		UFractureToolAutoUV* AutoUVTool = Cast<UFractureToolAutoUV>(OwnerTool.Get());
		AutoUVTool->UpdateUVChannels(Target);
	}
}

void UFractureAutoUVSettings::DisableBoneColors()
{
	UFractureToolAutoUV* AutoUVTool = Cast<UFractureToolAutoUV>(OwnerTool.Get());
	AutoUVTool->DisableBoneColors();
}

void UFractureAutoUVSettings::BoxProjectUVs()
{
	UFractureToolAutoUV* AutoUVTool = Cast<UFractureToolAutoUV>(OwnerTool.Get());
	AutoUVTool->BoxProjectUVs();
}

void UFractureAutoUVSettings::LayoutUVs()
{
	UFractureToolAutoUV* AutoUVTool = Cast<UFractureToolAutoUV>(OwnerTool.Get());
	AutoUVTool->LayoutUVs();
}

void UFractureAutoUVSettings::BakeTexture()
{
	UFractureToolAutoUV* AutoUVTool = Cast<UFractureToolAutoUV>(OwnerTool.Get());
	AutoUVTool->BakeTexture();
}

UFractureToolAutoUV::UFractureToolAutoUV(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	AutoUVSettings = NewObject<UFractureAutoUVSettings>(GetTransientPackage(), UFractureAutoUVSettings::StaticClass());
	AutoUVSettings->OwnerTool = this;
}

bool UFractureToolAutoUV::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}

	return true;
}

FText UFractureToolAutoUV::GetDisplayText() const
{
	return FText(LOCTEXT("FractureToolAutoUV", "AutoUV Fracture"));
}

FText UFractureToolAutoUV::GetTooltipText() const
{
	return FText(LOCTEXT("FractureToolAutoUVTooltip", "This enables you to automatically layout UVs for internal fracture pieces, and procedurally fill a corresponding texture."));
}

FSlateIcon UFractureToolAutoUV::GetToolIcon() const
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.AutoUV");
}

void UFractureToolAutoUV::RegisterUICommand(FFractureEditorCommands* BindingContext)
{
	UI_COMMAND_EXT(BindingContext, UICommandInfo, "AutoUV", "AutoUV", "Autogenerate UVs and textures for geometry collections (especially for internal fracture surfaces).", EUserInterfaceActionType::ToggleButton, FInputChord());
	BindingContext->AutoUV = UICommandInfo;
}

TArray<UObject*> UFractureToolAutoUV::GetSettingsObjects() const
{
	TArray<UObject*> Settings;
	Settings.Add(AutoUVSettings);
	return Settings;
}

void UFractureToolAutoUV::FractureContextChanged()
{
	UpdateUVChannels(-1);
}

void UFractureToolAutoUV::UpdateUVChannels(int32 TargetNumUVChannels)
{
	int32 MinUVChannels = GeometryCollectionUV::MAX_NUM_UV_CHANNELS;

	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			const int32 NumChannels = GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection()->NumUVLayers();
			MinUVChannels = FMath::Min(MinUVChannels, NumChannels);
		}
	}

	if (TargetNumUVChannels > -1 && TargetNumUVChannels != MinUVChannels)
	{
		FScopedTransaction Transaction(LOCTEXT("UpdateUVChannels", "Update UV Channels"), !GeomCompSelection.IsEmpty());

		bool bIsIncreasing = TargetNumUVChannels > MinUVChannels;
		for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
		{
			if (GeometryCollectionComponent->GetRestCollection())
			{
				const int32 NumChannels = GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection()->NumUVLayers();
				// not if num channels is decreasing, all channels will be >= min channels so we apply the size change to all
				// but if we're increasing the number of channels, some geometry collections may have more channels already and we leave those alone
				if (NumChannels < TargetNumUVChannels || !bIsIncreasing)
				{
					FGeometryCollectionEdit Edit(GeometryCollectionComponent, GeometryCollection::EEditUpdate::Rest, true);
					GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection()->SetNumUVLayers(TargetNumUVChannels);
					if (bIsIncreasing)
					{
						// Copy UV layer 0 into new UV layer
						FGeometryCollection& Collection = *GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection();
						GeometryCollection::UV::FUVLayers Layers = GeometryCollection::UV::FindActiveUVLayers(Collection);
						for (int32 Idx = 0; Idx < Layers[0].Num(); Idx++)
						{
							for (int32 Ch = NumChannels; Ch < TargetNumUVChannels; Ch++)
							{
								Layers[Ch][Idx] = Layers[0][Idx];
							}
						}
					}
					Edit.GetRestCollection()->RebuildRenderData();
				}
			}
		}
		MinUVChannels = TargetNumUVChannels;
	}

	AutoUVSettings->SetNumUVChannels(MinUVChannels);
}

void UFractureToolAutoUV::DisableBoneColors()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	FScopedTransaction Transaction(LOCTEXT("DisableBoneColors", "Disable Bone Colors"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			GeometryCollectionComponent->Modify();
			GeometryCollectionComponent->SetShowBoneColors(false);
			GeometryCollectionComponent->MarkRenderStateDirty();
		}
	}
}

void UFractureToolAutoUV::BoxProjectUVs()
{
	int32 UVLayer = AutoUVSettings->GetSelectedChannelIndex();

	TArray<int32> EmptyMaterialIDs;
	UE::PlanarCut::ETargetFaces TargetFaces = ConvertTargetFacesEnum(AutoUVSettings->TargetFaces);

	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	FScopedTransaction Transaction(LOCTEXT("BoxProjectUVs", "Box Project UVs"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			FGeometryCollectionEdit Edit(GeometryCollectionComponent, GeometryCollection::EEditUpdate::Rest, true);
			UE::PlanarCut::BoxProjectUVs(UVLayer,
				*GeometryCollectionComponent->GetRestCollection()->GetGeometryCollection(),
				(FVector3d)AutoUVSettings->ProjectionScale,
				TargetFaces,
				!TargetsMaterialIDs(AutoUVSettings->TargetFaces) ? EmptyMaterialIDs : AutoUVSettings->MaterialIDs,
				(FVector2f)AutoUVSettings->ProjectionUVOffset,
				AutoUVSettings->bAutoFitToBounds, AutoUVSettings->bCenterAtPivot, AutoUVSettings->bUniformProjectionScale);

			Edit.GetRestCollection()->RebuildRenderData();
		}
	}
}

void UFractureToolAutoUV::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
}

void UFractureToolAutoUV::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// update any cached data 
}

TArray<FFractureToolContext> UFractureToolAutoUV::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component, or for each individual bone if Group Fracture is not used.
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			// Generate a context for each selected node
			FFractureToolContext FullSelection(GeometryCollectionComponent);
			FullSelection.ConvertSelectionToRigidNodes();

			// Update global transforms and bounds -- TODO: pull this bounds update out to a shared function?
			const TManagedArray<FTransform3f>& Transform = FullSelection.GetGeometryCollection()->Transform;
			const TManagedArray<int32>& TransformToGeometryIndex = FullSelection.GetGeometryCollection()->TransformToGeometryIndex;
			const TManagedArray<FBox>& BoundingBoxes = FullSelection.GetGeometryCollection()->BoundingBox;

			TArray<FTransform> Transforms;
			GeometryCollectionAlgo::GlobalMatrices(Transform, FullSelection.GetGeometryCollection()->Parent, Transforms);

			FBox Bounds(ForceInit);
			for (int32 BoneIndex : FullSelection.GetSelection())
			{
				int32 GeometryIndex = TransformToGeometryIndex[BoneIndex];
				if (GeometryIndex > INDEX_NONE)
				{
					FBox BoneBound = BoundingBoxes[GeometryIndex].TransformBy(Transforms[BoneIndex]);
					Bounds += BoneBound;
				}
			}
			FullSelection.SetBounds(Bounds);

			Contexts.Add(FullSelection);
		}
	}

	return Contexts;
}


bool UFractureToolAutoUV::SaveGeneratedTexture(UE::Geometry::TImageBuilder<FVector4f>& ImageBuilder, FString ObjectBaseName, const UObject* RelativeToAsset, bool bPromptToSave, bool bAllowReplace)
{
	check(RelativeToAsset);

	// find path to reference asset
	UPackage* AssetOuterPackage = CastChecked<UPackage>(RelativeToAsset->GetOuter());
	FString AssetPackageName = AssetOuterPackage->GetName();
	FString PackageFolderPath = FPackageName::GetLongPackagePath(AssetPackageName);

	// Show the modal dialog and then get the path/name.
	// If the user cancels, texture is left as transient
	if (bPromptToSave)
	{
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

		FString UseDefaultAssetName = ObjectBaseName;
		FString CurrentPath = PackageFolderPath;
		if (!CurrentPath.IsEmpty() && !bAllowReplace)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			FString UnusedPackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(FPaths::Combine(PackageFolderPath, ObjectBaseName), TEXT(""), UnusedPackageName, UseDefaultAssetName);
		}

		FSaveAssetDialogConfig Config;
		Config.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		Config.DefaultAssetName = UseDefaultAssetName;
		Config.DialogTitleOverride = LOCTEXT("GenerateTexture2DAssetPathDialogWarning", "Choose Folder Path and Name for New Asset. Cancel to Discard New Asset.");
		Config.DefaultPath = CurrentPath;
		FString SelectedPath = ContentBrowser.CreateModalSaveAssetDialog(Config);

		if (SelectedPath.IsEmpty() == false)
		{
			PackageFolderPath = FPaths::GetPath(SelectedPath);
			ObjectBaseName = FPaths::GetBaseFilename(SelectedPath, true);
		}
		else
		{
			return false;
		}
	}

	FString NewAssetPath = FPaths::Combine(PackageFolderPath, ObjectBaseName);

	FTexture2DBuilder::ETextureType TextureType = FTexture2DBuilder::ETextureType::ColorLinear;
	if (AutoUVSettings->BakeTextureType == ETextureType::Normals)
	{
		TextureType = FTexture2DBuilder::ETextureType::NormalMap;
	}

	UTexture2D* GeneratedTexture = nullptr;
	bool bNeedsNewPackage = true;
	if (bAllowReplace)
	{
		// Modifying the static mesh in place. Delete existing asset so that we can have a clean duplicate
		bool bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewAssetPath);
		if (bNewAssetExists)
		{
			UObject* OldObject = UEditorAssetLibrary::LoadAsset(NewAssetPath);
			if (UTexture2D* OldTexture = Cast<UTexture2D>(OldObject))
			{
				FTexture2DBuilder TextureBuilder;
				TextureBuilder.InitializeAndReplaceExistingTexture(OldTexture, TextureType, ImageBuilder.GetDimensions());
				TextureBuilder.Copy(ImageBuilder);
				TextureBuilder.Commit(false);
				GeneratedTexture = TextureBuilder.GetTexture2D();
				FTexture2DBuilder::CopyPlatformDataToSourceData(GeneratedTexture, TextureType);
				bNeedsNewPackage = false;
			}
			else // old asset was wrong type; delete to replace
			{
				bool bDeleteOK = UEditorAssetLibrary::DeleteAsset(NewAssetPath);
				ensure(bDeleteOK);
			}
		}
	}
	if (!GeneratedTexture)
	{
		FTexture2DBuilder TextureBuilder;
		TextureBuilder.Initialize(TextureType, ImageBuilder.GetDimensions());
		TextureBuilder.Copy(ImageBuilder);
		TextureBuilder.Commit(false);
		GeneratedTexture = TextureBuilder.GetTexture2D();
		FTexture2DBuilder::CopyPlatformDataToSourceData(GeneratedTexture, TextureType);
	}
	AutoUVSettings->Result = GeneratedTexture;
	check(GeneratedTexture);
	check(GeneratedTexture->Source.IsValid());	// texture needs to have valid source data to be savd

	if (bNeedsNewPackage)
	{
		check(GeneratedTexture->GetOuter() == GetTransientPackage());

		// create new package
		FString UniqueAssetName;
		FString UniquePackageName;

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(NewAssetPath, TEXT(""), UniquePackageName, UniqueAssetName);

		UPackage* AssetPackage = CreatePackage(*UniquePackageName);

		// move texture from Transient package to real package
		GeneratedTexture->Rename(*UniqueAssetName, AssetPackage, REN_None);
	}
	// remove transient flag, add public/standalone/transactional
	GeneratedTexture->ClearFlags(RF_Transient);
	GeneratedTexture->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
	// mark things as modified / dirtied
	GeneratedTexture->Modify();
	GeneratedTexture->UpdateResource();
	GeneratedTexture->PostEditChange();
	GeneratedTexture->MarkPackageDirty();

	FAssetRegistryModule::AssetCreated(GeneratedTexture);

	return true;
}

 class FLayoutUVOp : public FGeometryCollectionOperator
 {
 public:
 	FLayoutUVOp(const FGeometryCollection& SourceCollection) : FGeometryCollectionOperator(SourceCollection)
 	{}

 	virtual ~FLayoutUVOp() = default;

	int32 UVLayer;
	int32 OutputRes;
	int32 GutterSize;
	UE::PlanarCut::ETargetFaces TargetFaces;
	TArray<int32> MaterialIDs;

 	// TGenericDataOperator interface:
 	virtual void CalculateResult(FProgressCancel* Progress) override
 	{
		bool bLayoutSuccess =
			UE::PlanarCut::UVLayout(UVLayer, *CollectionCopy, OutputRes, GutterSize,
				TargetFaces, MaterialIDs, true, Progress);
		
		if (bLayoutSuccess)
		{
			SetSuccessIndex();
			SetResult(MoveTemp(CollectionCopy));
		}
 	}
 };


void UFractureToolAutoUV::LayoutUVs()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	FScopedTransaction Transaction(LOCTEXT("LayoutUVs", "Layout UVs"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			FGeometryCollectionEdit Edit(GeometryCollectionComponent, GeometryCollection::EEditUpdate::None);
			Edit.GetRestCollection()->Modify();
			LayoutUVsForComponent(GeometryCollectionComponent);

			Edit.GetRestCollection()->RebuildRenderData();
		}
	}
}


bool UFractureToolAutoUV::LayoutUVsForComponent(UGeometryCollectionComponent* Component)
{
	FGeometryCollection& Collection = *Component->GetRestCollection()->GetGeometryCollection();

	TUniquePtr<FLayoutUVOp> LayoutOp = MakeUnique<FLayoutUVOp>(Collection);
	LayoutOp->OutputRes = (int32)AutoUVSettings->Resolution;
	TArray<int32> EmptyMaterialIDs;
	LayoutOp->MaterialIDs = !TargetsMaterialIDs(AutoUVSettings->TargetFaces) ? EmptyMaterialIDs : AutoUVSettings->MaterialIDs;

	LayoutOp->OutputRes = (int32)AutoUVSettings->Resolution;
	LayoutOp->UVLayer = AutoUVSettings->GetSelectedChannelIndex();
	LayoutOp->GutterSize = AutoUVSettings->GutterSize;
	LayoutOp->TargetFaces = ConvertTargetFacesEnum(AutoUVSettings->TargetFaces);

	int Result = RunCancellableGeometryCollectionOp<FLayoutUVOp>(Collection,
		MoveTemp(LayoutOp), LOCTEXT("ComputingUVLayoutMessage", "Computing UV Layout"));
	return Result != -1;
}


class FBakeTextureOp : public TGeometryCollectionOperator<UE::Geometry::TImageBuilder<FVector4f>>
{
public:
	FBakeTextureOp(const FGeometryCollection& SourceCollection) : TGeometryCollectionOperator<UE::Geometry::TImageBuilder<FVector4f>>(SourceCollection)
	{}

	virtual ~FBakeTextureOp() = default;

	int32 UVLayer;
	int32 GutterSize;
	FIndex4i Attributes;
	UE::PlanarCut::FTextureAttributeSettings AttribSettings;
	TUniquePtr<UE::Geometry::TImageBuilder<FVector4f>> ImageBuilder;
	UE::PlanarCut::ETargetFaces TargetFaces;
	TArray<int32> MaterialIDs;

	// TGenericDataOperator interface:
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		bool bBakeSuccess = UE::PlanarCut::TextureSpecifiedFaces(
			UVLayer, *CollectionCopy, GutterSize, Attributes, AttribSettings, *ImageBuilder,
			TargetFaces, MaterialIDs, Progress);

		if (!bBakeSuccess || (Progress && Progress->Cancelled()))
		{
			return;
		}

		SetResult(MoveTemp(ImageBuilder));
		SetSuccessIndex();
	}
};


void UFractureToolAutoUV::BakeTexture()
{
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);

	FScopedTransaction Transaction(LOCTEXT("BakeTexture", "Bake Texture"), !GeomCompSelection.IsEmpty());
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		if (GeometryCollectionComponent->GetRestCollection())
		{
			BakeTextureForComponent(GeometryCollectionComponent);
		}
	}
}

void UFractureToolAutoUV::BakeTextureForComponent(UGeometryCollectionComponent* Component)
{
	using FImageType = UE::Geometry::TImageBuilder<FVector4f>;

	FGeometryCollection& Collection = *Component->GetRestCollection()->GetGeometryCollection();

	TUniquePtr<FBakeTextureOp> BakeOp = MakeUnique<FBakeTextureOp>(Collection);
	TArray<int32> EmptyMaterialIDs;
	BakeOp->MaterialIDs = !TargetsMaterialIDs(AutoUVSettings->TargetFaces) ? EmptyMaterialIDs : AutoUVSettings->MaterialIDs;

	int32 OutputRes = (int32)AutoUVSettings->Resolution;
	BakeOp->UVLayer = AutoUVSettings->GetSelectedChannelIndex();
	BakeOp->GutterSize = AutoUVSettings->GutterSize + (AutoUVSettings->bFullyPadGutterOnBake ? OutputRes : AutoUVSettings->GutterPaddingOnBake);
	BakeOp->TargetFaces = ConvertTargetFacesEnum(AutoUVSettings->TargetFaces);

	FImageDimensions Dimensions(OutputRes, OutputRes);
	BakeOp->ImageBuilder = MakeUnique<FImageType>();
	BakeOp->ImageBuilder->SetDimensions(Dimensions);
	BakeOp->ImageBuilder->Clear(FVector4f(0, 0, 0, 0));

	typedef UE::PlanarCut::EBakeAttributes EBakeAttributes;
	// Note: Ordering of these attributes should match the order and comments in the AutoUVSettings struct
	//		 Update the order and comments there if you change the ordering here.
	if (AutoUVSettings->BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes)
	{
		BakeOp->Attributes = FIndex4i(
			AutoUVSettings->bDistToOuter ? (int32)EBakeAttributes::DistanceToExternal : 0,
			AutoUVSettings->bAmbientOcclusion ? (int32)EBakeAttributes::AmbientOcclusion : 0,
			AutoUVSettings->bSmoothedCurvature ? (int32)EBakeAttributes::Curvature : 0,
			0
		);
	}
	else if (AutoUVSettings->BakeTextureType == ETextureType::SpatialGradients)
	{
		BakeOp->Attributes = FIndex4i(
			(int32)EBakeAttributes::PositionX,
			(int32)EBakeAttributes::PositionY,
			(int32)EBakeAttributes::PositionZ,
			0
		);
	}
	else // ETextureType::Normals
	{
		BakeOp->Attributes = FIndex4i(
			(int32)EBakeAttributes::NormalX,
			(int32)EBakeAttributes::NormalY,
			(int32)EBakeAttributes::NormalZ,
			0
		);
	}
	BakeOp->AttribSettings.ToExternal_MaxDistance = AutoUVSettings->MaxDistance;
	BakeOp->AttribSettings.AO_Rays = AutoUVSettings->OcclusionRays;
	BakeOp->AttribSettings.AO_BlurRadius = AutoUVSettings->OcclusionBlurRadius;
	BakeOp->AttribSettings.Curvature_BlurRadius = AutoUVSettings->CurvatureBlurRadius;
	BakeOp->AttribSettings.Curvature_SmoothingSteps = AutoUVSettings->SmoothingIterations;
	BakeOp->AttribSettings.Curvature_VoxelRes = AutoUVSettings->VoxelResolution;
	BakeOp->AttribSettings.Curvature_ThicknessFactor = AutoUVSettings->ThicknessFactor;
	BakeOp->AttribSettings.Curvature_MaxValue = AutoUVSettings->MaxCurvature;
	BakeOp->AttribSettings.ClearGutterChannel = AutoUVSettings->bClearGutterAlpha ? 3 : INDEX_NONE;

	FImageType ImageResult;
	int Result = RunCancellableGeometryCollectionOpGeneric<FBakeTextureOp, FImageType>(
		ImageResult, [](FImageType& ToUpdate, FImageType& Result)
		{
			ToUpdate = MoveTemp(Result);
		},
		MoveTemp(BakeOp), LOCTEXT("ComputingBakeTextureMessage", "Baking Textures"));
	if (Result == -1)
	{
		// Bake failed or was cancelled
		return;
	}

	// choose default texture name based on corresponding geometry collection name
	FString BaseName = Component->GetRestCollection()->GetName();
	FString Suffix = "_AutoUV";
	if (AutoUVSettings->BakeTextureType == ETextureType::SpatialGradients)
	{
		Suffix = "_AutoUV_Spatial";
	}
	else if (AutoUVSettings->BakeTextureType == ETextureType::Normals)
	{
		Suffix = "_AutoUV_Normals";
	}
	Suffix = FPaths::MakeValidFileName(Suffix);
	SaveGeneratedTexture(ImageResult, FString::Printf(TEXT("%s%s"), *BaseName, *Suffix), Component->GetRestCollection(), AutoUVSettings->bPromptToSave, AutoUVSettings->bReplaceExisting);
}



int32 UFractureToolAutoUV::ExecuteFracture(const FFractureToolContext& FractureContext)
{
	if (FractureContext.GetGeometryCollection().IsValid())
	{
		bool bDoUVLayout = true;

		UGeometryCollectionComponent* Component = FractureContext.GetGeometryCollectionComponent();

		if (bDoUVLayout)
		{
			if (!LayoutUVsForComponent(Component))
			{
				return INDEX_NONE; // Early out if layout failed or was cancelled
			}
		}

		BakeTextureForComponent(Component);
	}

	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE

