// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureTool.h"
#include "GeometryCollection/GeometryCollection.h"

#include "Image/ImageBuilder.h"

#include "FractureToolUV.generated.h"

class FFractureToolContext;

UENUM()
enum class EAutoUVTextureResolution : int32
{
	Resolution16 = 16 UMETA(DisplayName = "16 x 16"),
	Resolution32 = 32 UMETA(DisplayName = "32 x 32"),
	Resolution64 = 64 UMETA(DisplayName = "64 x 64"),
	Resolution128 = 128 UMETA(DisplayName = "128 x 128"),
	Resolution256 = 256 UMETA(DisplayName = "256 x 256"),
	Resolution512 = 512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 = 1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 = 2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 = 4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 = 8192 UMETA(DisplayName = "8192 x 8192")
};

UENUM()
enum class ETextureType
{
	// Bake thickness, ambient occlusion, and a smoothed curvature metric
	ThicknessAndSurfaceAttributes,
	// Bake position relative to the bounding box
	SpatialGradients,
	// Bake surface normals
	Normals
};

UENUM()
enum class ETargetFaces
{
	// Target internal faces
	InternalFaces,
	// Target internal faces and those with material IDs matching a custom list
	InternalFacesAndSelectedMaterialIDs,
	// Target faces with material IDs matching a custom list
	SelectedMaterialIDs,
	// Target external faces
	ExternalFaces,
	// Target all faces
	AllFaces
};

/** Settings for UV layout and texture baking on the geometry collection **/
UCLASS(config = EditorPerProjectUserSettings)
class UFractureAutoUVSettings : public UFractureToolSettings
{
public:

	GENERATED_BODY()

	UFractureAutoUVSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		SetNumUVChannels(1);
	}

	/** Which UV channel to use for layout and baking */
	UPROPERTY(EditAnywhere, Category = UVChannel, meta = (DisplayName = "UV Channel", GetOptions = GetUVChannelNamesFunc))
	FString UVChannel;

	UFUNCTION()
	const TArray<FString>& GetUVChannelNamesFunc()
	{
		return UVChannelNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVChannelNamesList;

	void SetNumUVChannels(int32 NumUVChannels);
	int32 GetSelectedChannelIndex(bool bForceToZeroOnFailure = true);

	/** Add a UV channel to the Geometry Collection */
	UFUNCTION(CallInEditor, Category = UVChannel, meta = (DisplayName = "Add UV Channel"))
	void AddUVChannel()
	{
		ChangeNumUVChannels(1);
	}

	/** Remove the last UV channel from the Geometry Collection */
	UFUNCTION(CallInEditor, Category = UVChannel, meta = (DisplayName = "Delete UV Channel"))
	void DeleteUVChannel()
	{
		ChangeNumUVChannels(-1);
	}

	void ChangeNumUVChannels(int32 Delta);

	/** Immediately layout and update the UVs of the selected Geometry Collection, for faces with the Target Material IDs */
	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayName = "Layout UVs", DisplayPriority = 1))
	void LayoutUVs();

	/** Bake a new texture with the chosen attributes, aligned to the current UV layout */
	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayName = "Bake Texture", DisplayPriority = 2))
	void BakeTexture();

	/** Disable "Show Bone Colors" on the selected Geometry Collection Components, so textures are visible. This setting can also be toggled directly on the Component itself */
	UFUNCTION(CallInEditor, Category = Operations)
	void DisableBoneColors();

	/** The scale factor to use for UV box projection */
	UPROPERTY(EditAnywhere, Category = Unwrap, meta = (EditCondition = "!bAutoFitToBounds"))
	FVector ProjectionScale = FVector(100, 100, 100);

	/** Set the scale factor for UV box projection based on the bounding box of the geometry */
	UPROPERTY(EditAnywhere, Category = Unwrap, AdvancedDisplay)
	bool bAutoFitToBounds = false;

	/** Ensure the projection scale is the same on each axis, either by repeating the X Axis scale, or using the max box dimension if Auto Fit to Bounds is selected. */
	UPROPERTY(EditAnywhere, Category = Unwrap, AdvancedDisplay, meta = (DisplayName = "Uniform Scale"))
	bool bUniformProjectionScale = false;

	/** UV Offset to apply after UV box projection */
	UPROPERTY(EditAnywhere, Category = Unwrap, AdvancedDisplay, meta = (DisplayName = "UV Offset"))
	FVector2D ProjectionUVOffset = FVector2D(.5, .5);

	/** Optionally center the UV projection around the object pivot */
	UPROPERTY(EditAnywhere, Category = Unwrap, AdvancedDisplay)
	bool bCenterAtPivot = false;

	/** Immediately perform a box projection to update the internal UVs of the selected Geometry Collection */
	UFUNCTION(CallInEditor, Category = Unwrap, meta = (DisplayName = "Box Project UVs"))
	void BoxProjectUVs();

	/** Choose whether to target internal faces, or a custom selection */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	ETargetFaces TargetFaces;

	/** Custom selection of material IDs to target for texturing */
	UPROPERTY(EditAnywhere, Category = MapSettings, meta = (EditCondition = "TargetFaces != ETargetFaces::InternalFaces && TargetFaces != ETargetFaces::AllFaces && TargetFaces != ETargetFaces::ExternalFaces", EditConditionHides))
	TArray<int32> MaterialIDs;

	/** The pixel resolution of the generated map */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	EAutoUVTextureResolution Resolution = EAutoUVTextureResolution::Resolution512;

	/** Approximate space to leave between UV islands, measured in texels */
	UPROPERTY(EditAnywhere, Category = MapSettings, meta = (ClampMin = "1", ClampMax = "10", UIMax = "6"))
	int32 GutterSize = 2;

	/** Whether to set gutter texels to transparent, or to copy them unchanged from the closest interior texel value. */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	bool bClearGutterAlpha = false;

	/** Whether to extend copied border texel values into all empty texture space, beyond the Gutter Size. Only applied when baking, does not affect spacing for UV Layout. */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	bool bFullyPadGutterOnBake = true;

	/** Amount of texels to extend copied border texel values into empty texture space, beyond the Gutter Size. Only applied when baking, does not affect spacing for UV Layout. */
	UPROPERTY(EditAnywhere, Category = MapSettings, meta = (ClampMin = "0", UIMax = "10", EditCondition="!bFullyPadGutterOnBake"))
	int32 GutterPaddingOnBake = 4;

	/** The resulting automatically-generated texture map */
	UPROPERTY(VisibleAnywhere, Category = MapSettings)
	TObjectPtr<UTexture2D> Result;

	/** Whether to prompt user for an asset name for each generated texture, or automatically place them next to the source Geometry Collections */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	bool bPromptToSave = true;

	/** Whether to allow the new texture to overwrite an existing texture */
	UPROPERTY(EditAnywhere, Category = MapSettings)
	bool bReplaceExisting = true;

	/** Which standard set of texture channels to bake */
	UPROPERTY(EditAnywhere, Category = AttributesToBake)
	ETextureType BakeTextureType = ETextureType::ThicknessAndSurfaceAttributes;

	/** Bake the distance to the external surface to a texture channel (red) */
	UPROPERTY(EditAnywhere, Category = AttributesToBake, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes", EditConditionHides))
	bool bDistToOuter = true;

	/** Bake the ambient occlusion of each bone (considered separately) to a texture channel (green) */
	UPROPERTY(EditAnywhere, Category = AttributesToBake, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes", EditConditionHides))
	bool bAmbientOcclusion = true;

	/**
	 * Bake a smoothed curvature metric to a texture channel (blue)
	 * Specifically, this is the mean curvature of a smoothed copy of each fractured piece, baked back to the respective fracture piece
	 * Note that this attribute is relatively expensive to compute
	 */
	UPROPERTY(EditAnywhere, Category = AttributesToBake, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes", EditConditionHides))
	bool bSmoothedCurvature = false;

	/** Max distance to search for the outer mesh surface */
	UPROPERTY(EditAnywhere, Category = DistToOuterSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bDistToOuter", EditConditionHides, UIMin = "1", UIMax = "100", ClampMin = ".01", ClampMax = "1000"))
	double MaxDistance = 100;

	/** Number of occlusion rays */
	UPROPERTY(EditAnywhere, Category = AmbientOcclusionSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bAmbientOcclusion", EditConditionHides, UIMin = "1", UIMax = "1024", ClampMin = "0", ClampMax = "50000"))
	int OcclusionRays = 16;

	/** Pixel Radius of Gaussian Blur Kernel applied to AO map (0 will apply no blur) */
	UPROPERTY(EditAnywhere, Category = AmbientOcclusionSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bAmbientOcclusion", EditConditionHides, UIMin = "0", UIMax = "10.0", ClampMin = "0", ClampMax = "100.0"))
	double OcclusionBlurRadius = 2.25;

	/** Pixel Radius of Gaussian Blur Kernel applied to Curvature map (0 will apply no blur) */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bSmoothedCurvature", EditConditionHides, UIMin = "0", UIMax = "10.0", ClampMin = "0", ClampMax = "100.0"))
	double CurvatureBlurRadius = 2.25;

	/** Voxel resolution of smoothed shape representation */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bSmoothedCurvature", EditConditionHides, UIMin = "8", UIMax = "512", ClampMin = "4", ClampMax = "1024"))
	int VoxelResolution = 128;

	/** Amount of smoothing iterations to apply before computing curvature */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bSmoothedCurvature", EditConditionHides, UIMin = "2", UIMax = "100", ClampMin = "2", ClampMax = "1000"))
	int SmoothingIterations = 10;

	/** Distance to search for correspondence between fractured shape and smoothed shape, as factor of voxel size */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bSmoothedCurvature", EditConditionHides, UIMin = "2", UIMax = "10.0", ClampMin = "1", ClampMax = "100.0"))
	double ThicknessFactor = 4;

	/** Curvatures in the range [-MaxCurvature, MaxCurvature] will be mapped from [0,1]. Values outside that range will be clamped */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (EditCondition = "BakeTextureType == ETextureType::ThicknessAndSurfaceAttributes && bSmoothedCurvature", EditConditionHides, UIMin = ".01", UIMax = "1", ClampMin = ".0001", ClampMax = "10"))
	double MaxCurvature = .1;

};


UCLASS(DisplayName = "AutoUV Tool", Category = "FractureTools")
class UFractureToolAutoUV : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureToolAutoUV(const FObjectInitializer& ObjInit);

	///
	/// UFractureModalTool Interface
	///

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const override { return FText(NSLOCTEXT("AutoUV", "ExecuteAutoUV", "Layout & Bake")); }
	virtual FSlateIcon GetToolIcon() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void FractureContextChanged() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) override;
	virtual bool CanExecute() const override;
	virtual bool ExecuteUpdatesShape() const override
	{
		return false;
	}
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Gets the UI command info for this command */
	const TSharedPtr<FUICommandInfo>& GetUICommandInfo() const;

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const override;

	// Update number of UV channels for selected geometry collections
	void UpdateUVChannels(int32 TargetNumUVChannels);

	// Create new UV islands via box projection for the selected geometry collections
	void BoxProjectUVs();

	void DisableBoneColors();
	
	void LayoutUVs();
	void BakeTexture();


protected:
	UPROPERTY(EditAnywhere, Category = Slicing)
	TObjectPtr<UFractureAutoUVSettings> AutoUVSettings;

	bool SaveGeneratedTexture(UE::Geometry::TImageBuilder<FVector4f>& ImageBuilder, FString ObjectBaseName, const UObject* RelativeToAsset, bool bPromptToSave, bool bAllowReplace);

	bool LayoutUVsForComponent(UGeometryCollectionComponent* Component);
	void BakeTextureForComponent(UGeometryCollectionComponent* Component);
};
