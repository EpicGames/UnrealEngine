// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureToolUniform.h"

#include "FractureEditorStyle.h"
#include "FractureEditorCommands.h"
#include "FractureToolContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureToolUniform)

#define LOCTEXT_NAMESPACE "FractureUniform"


UFractureToolUniform::UFractureToolUniform(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
	UniformSettings = NewObject<UFractureUniformSettings>(GetTransientPackage(), UFractureUniformSettings::StaticClass());
	UniformSettings->OwnerTool = this;
}

FText UFractureToolUniform::GetDisplayText() const
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolUniform", "Uniform Voronoi Fracture")); 
}

FText UFractureToolUniform::GetTooltipText() const 
{ 
	return FText(NSLOCTEXT("Fracture", "FractureToolUniformTooltip", "Uniform Fracture will create pieces of approximately the same volume.  Specify minimum and maximum number of sites.  Using the same Random Seed will produce the same fracture.")); 
}

FSlateIcon UFractureToolUniform::GetToolIcon() const 
{
	return FSlateIcon("FractureEditorStyle", "FractureEditor.Uniform");
}

void UFractureToolUniform::RegisterUICommand( FFractureEditorCommands* BindingContext ) 
{
	UI_COMMAND_EXT( BindingContext, UICommandInfo, "Uniform", "Uniform", "Fracture using a Voronoi diagram with a uniform random pattern, creating fracture pieces of similar volume across the shape.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	BindingContext->Uniform = UICommandInfo;
}

TArray<UObject*> UFractureToolUniform::GetSettingsObjects() const 
{ 
	TArray<UObject*> AllSettings; 
	AllSettings.Add(UniformSettings);
	AllSettings.Add(CutterSettings);
	AllSettings.Add(CollisionSettings);
	return AllSettings;
}

void UFractureToolUniform::GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites)
{
	FRandomStream RandStream(Context.GetSeed());

	FBox Bounds = Context.GetWorldBounds();
	const FVector Extent(Bounds.Max - Bounds.Min);

	const int32 SiteCount = RandStream.RandRange(UniformSettings->NumberVoronoiSitesMin, UniformSettings->NumberVoronoiSitesMax);

	Sites.Reserve(Sites.Num() + SiteCount);
	for (int32 ii = 0; ii < SiteCount; ++ii)
	{
		Sites.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent );
	}
}




#undef LOCTEXT_NAMESPACE

