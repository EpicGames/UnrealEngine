// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FractureToolCutter.h"

#include "FractureToolUniform.generated.h"

class FFractureToolContext;

UCLASS(config = EditorPerProjectUserSettings)
class UFractureUniformSettings : public UFractureToolSettings
{
public:
	GENERATED_BODY()

	UFractureUniformSettings(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, NumberVoronoiSitesMin(20)
		, NumberVoronoiSitesMax(20)
	{}

	/** Minimum Number of Voronoi sites. The amount of sites per Voronoi diagram will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = UniformVoronoi, meta = (DisplayName = "Min Voronoi Sites", UIMin = "1", UIMax = "5000", ClampMin = "1"))
	int32 NumberVoronoiSitesMin;

	/** Maximum Number of Voronoi sites. The amount of sites per Voronoi diagram will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = UniformVoronoi, meta = (DisplayName = "Max Voronoi Sites", UIMin = "1", UIMax = "5000", ClampMin = "1"))
	int32 NumberVoronoiSitesMax;

};


UCLASS(DisplayName="Uniform Voronoi", Category="FractureTools")
class UFractureToolUniform : public UFractureToolVoronoiCutterBase
{
public:
	GENERATED_BODY()

	UFractureToolUniform(const FObjectInitializer& ObjInit);

	// UFractureTool Interface
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetToolIcon() const override;
	virtual TArray<UObject*> GetSettingsObjects() const override;
	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) override;


	// Uniform Voronoi Fracture Input Settings
	UPROPERTY(EditAnywhere, Category = Uniform)
	TObjectPtr<UFractureUniformSettings> UniformSettings;

protected:
	void GenerateVoronoiSites(const FFractureToolContext& Context, TArray<FVector>& Sites) override;
};

