// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionOutlinerDragDrop.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "ChaosEditor"

bool FGeometryCollectionBoneDragDrop::ValidateDrop(const FGeometryCollection* OtherGeometryCollection, int32 OtherBone, FText& MessageText)
{
	if (!OtherGeometryCollection)
	{
		return false;
	}

	// We don't currently support transfer of geometry from one collection to another.
	if (OtherGeometryCollection != GeometryCollection.Get())
	{
		MessageText = NSLOCTEXT("GeometryCollectionOutliner", "ComponentTransfer_GeometryCollectionOutliner", "Bones must remain within original component.");
		return false;
	}

	if (BonePayload.Contains(OtherBone))
	{
		MessageText = NSLOCTEXT("GeometryCollectionOutliner", "Self_GeometryCollectionOutliner", "Cannot parent to self.");
		return false;
	}

	const TManagedArray<int32>& ExemplarIndex = OtherGeometryCollection->ExemplarIndex;
	const TManagedArray<int32>& SimulationType = OtherGeometryCollection->SimulationType;
	if (OtherGeometryCollection->IsClustered(OtherBone))
	{
		if (ContainsInstance())
		{
			MessageText = NSLOCTEXT("GeometryCollectionOutliner", "Instances_GeometryCollectionOutliner",
				"Cannot parent instanced embedded geometry directly to cluster.");
			return false;
		}

		if (ContainsEmbedded())
		{
			MessageText = NSLOCTEXT("GeometryCollectionOutliner", "EmbeddedToCluster_GeometryCollectionOutliner",
				"Embedded geometry will convert to rigids when parented to cluster.");
			return true;
		}

		return true;
	}
	else if (OtherGeometryCollection->IsRigid(OtherBone))
	{
		if (ContainsCluster())
		{
			MessageText = NSLOCTEXT("GeometryCollectionOutliner", "ClusterToRigid_GeometryCollectionOutliner",
				"Cannot parent cluster to rigid.");
			return false;
		}

		if (ContainsRigid())
		{
			MessageText = NSLOCTEXT("GeometryCollectionOutliner", "RigidToRigid_GeometryCollectionOutliner",
				"Rigids will convert to embedded geometry when parented to rigid.");
			return true;
		}

		return true;
	}
	else if (SimulationType[OtherBone] == FGeometryCollection::ESimulationTypes::FST_None)
	{
		if (ContainsCluster() || ContainsRigid())
		{
			MessageText = NSLOCTEXT("GeometryCollectionOutliner", "ToEmbedded_GeometryCollectionOutliner",
				"Cannot parent to embedded geometry.");
			return false;
		}

		return true;
	}

	// We disallow any unrecognized combinations
	return false;
}

bool FGeometryCollectionBoneDragDrop::ReparentBones(const FGeometryCollection* OtherGeometryCollection, int32 OtherBone)
{
	FText HoverText;
	bool bValid = ValidateDrop(OtherGeometryCollection, OtherBone, HoverText);
	if (bValid)
	{
		FScopedTransaction Transaction(LOCTEXT("ReparentBones", "Reparent Bones"));
		
		// If we parent a rigid to another rigid, we convert the child rigid to embedded.
		if (GeometryCollection->IsRigid(OtherBone))
		{
			TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
			for (int32 Bone : BonePayload)
			{
				if (GeometryCollection->IsRigid(Bone))
				{
					SimulationType[Bone] = FGeometryCollection::ESimulationTypes::FST_None;
				}
			}
		}
		// If we parent an embedded geometry to a cluster, it becomes a rigid body.
		else if (GeometryCollection->IsClustered(OtherBone))
		{
			const TManagedArray<int32>& ExemplarIndex = OtherGeometryCollection->ExemplarIndex;
			TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
			for (int32 Bone : BonePayload)
			{
				if ((SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_None) && (ExemplarIndex[Bone] == INDEX_NONE))
				{
					SimulationType[Bone] = FGeometryCollection::ESimulationTypes::FST_Rigid;
				}
			}
		}

		FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(GeometryCollection.Get(), OtherBone, BonePayload);
		FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeometryCollection.Get());
	}

	return bValid;
}

bool FGeometryCollectionBoneDragDrop::ContainsCluster() const
{
	for (int32 Bone : BonePayload)
	{
		if (GeometryCollection->IsClustered(Bone))
		{
			return true;
		}
	}
	return false;
}

bool FGeometryCollectionBoneDragDrop::ContainsRigid() const
{
	for (int32 Bone : BonePayload)
	{
		if (GeometryCollection->IsRigid(Bone))
		{
			return true;
		}
	}
	return false;
}

bool FGeometryCollectionBoneDragDrop::ContainsEmbedded() const
{
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	for (int32 Bone : BonePayload)
	{
		if (SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			return true;
		}
	}
	return false;
}

bool FGeometryCollectionBoneDragDrop::ContainsInstance() const
{
	const TManagedArray<int32>& ExemplarIndex = GeometryCollection->ExemplarIndex;
	for (int32 Bone : BonePayload)
	{
		if (ExemplarIndex[Bone] > INDEX_NONE)
		{
			return true;
		}
	}
	return false;
}


#undef LOCTEXT_NAMESPACE