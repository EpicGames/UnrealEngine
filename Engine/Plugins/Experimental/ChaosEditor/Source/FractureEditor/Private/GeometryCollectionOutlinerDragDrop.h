// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

#include "SGeometryCollectionOutliner.h"

/**
* This operation is a bone which can be dragged and dropped to another bone in the outliner.
*/
class FGeometryCollectionBoneDragDrop
	: public FDecoratedDragDropOp
{
public:

	DRAG_DROP_OPERATOR_TYPE(FGeometryCollectionBoneDragDrop, FDecoratedDragDropOp)


	TArray<int32> BonePayload;
	TSharedPtr<FGeometryCollection,ESPMode::ThreadSafe> GeometryCollection;

	static TSharedRef<FGeometryCollectionBoneDragDrop> New(TSharedPtr<FGeometryCollection,ESPMode::ThreadSafe> InGeometryCollection, TArray<int32>& InBonePayload)
	{
		TSharedRef<FGeometryCollectionBoneDragDrop> Operation = MakeShareable(new FGeometryCollectionBoneDragDrop);
		Operation->MouseCursor = EMouseCursor::GrabHandClosed;
		Operation->BonePayload = InBonePayload;
		Operation->GeometryCollection = InGeometryCollection;

		Operation->Construct();
		return Operation;
	}

	/** Return true if dropping the BonePayload onto OtherBone is a valid operation. If it's not, return false and provide reason in MessageText. */
	bool ValidateDrop(const FGeometryCollection* OtherGeometryCollection, int32 OtherBone, FText& MessageText);

	/** Reparent BonePayload to OtherBone if this is a valid operation. Return false otherwise. */
	bool ReparentBones(const FGeometryCollection* OtherGeometryCollection, int32 OtherBone);

private:
	bool ContainsCluster() const;
	bool ContainsRigid() const;
	bool ContainsEmbedded() const;
	bool ContainsInstance() const;
	
};