// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCollectionHistogram.h"

#include "FractureToolProperties.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionUtility.h"

#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/STextBlock.h"

#include "FractureEditorMode.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SGeometryCollectionHistogram)

#define LOCTEXT_NAMESPACE "ChaosEditor"


UHistogramSettings::UHistogramSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, InspectedAttribute(EInspectedAttributeEnum::Volume)
	, bSorted(true)
{}

TSharedRef<ITableRow> FGeometryCollectionHistogramItem::MakeHistogramRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(STableRow<FGeometryCollectionHistogramItemPtr>, InOwnerTable)
		.Padding(FMargin(1.0f))
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(NormalizedValue)
			[
				SNew(SColorBlock)
				.Color(NodeColor)
				.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
				.Size(FVector2D(1.0f,5.0f))
				.ToolTipText(FText::FromString(HoverString))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0-NormalizedValue)
			[
				SNew(SBox)
			]
		];
}

void FGeometryCollectionHistogramItem::SetInspectedAttribute(EInspectedAttributeEnum InspectedAttribute)
{
	if (ensure(ParentComponentItem.IsValid()))
	{
		UGeometryCollectionComponent* Component = ParentComponentItem->GetComponent();

		if (Component)
		{
			if (FGeometryCollection* Collection = Component->GetRestCollection()->GetGeometryCollection().Get())
			{

				switch (InspectedAttribute)
				{
				case EInspectedAttributeEnum::Volume:
					{
						const TManagedArrayAccessor<float> VolumeAttribute(*Collection, "Volume", FGeometryCollection::TransformAttribute);
						InspectedValue = (VolumeAttribute.IsValid() && VolumeAttribute.IsValidIndex(BoneIndex)) ? VolumeAttribute[BoneIndex] : 0.0f;
						HoverString = FString::Printf(TEXT("%d: %.2f"), BoneIndex, InspectedValue);
					}
					break;
				case EInspectedAttributeEnum::Level:
					{
						const TManagedArrayAccessor<int32> LevelAttribute(*Collection, "Level", FGeometryCollection::TransformAttribute);
						const int32 Level = (LevelAttribute.IsValid() && LevelAttribute.IsValidIndex(BoneIndex)) ? LevelAttribute[BoneIndex] : 0;
						HoverString = FString::Printf(TEXT("%d: %d"), BoneIndex, Level);
						InspectedValue = static_cast<float>(Level);
					}
					break;

				case EInspectedAttributeEnum::InitialDynamicState:
					{
						const TManagedArrayAccessor<int32> InitialDynamicStateAttribute(*Collection, "InitialDynamicState", FGeometryCollection::TransformAttribute);
						const int32 InitialDynamicState = (InitialDynamicStateAttribute.IsValid() && InitialDynamicStateAttribute.IsValidIndex(BoneIndex)) ? InitialDynamicStateAttribute[BoneIndex] : 0;
						static const TArray<FString> HoverNames{ "No Override", "Sleeping", "Kinematic", "Static" };
						HoverString = FString::Printf(TEXT("%d: %s"), BoneIndex, *HoverNames[InitialDynamicState]);
						InspectedValue = static_cast<float>(InitialDynamicState);
					}
					break;

				case EInspectedAttributeEnum::Size:
					{
						const TManagedArrayAccessor<float> SizeAttribute(*Collection, "Size", FGeometryCollection::TransformAttribute);
						InspectedValue = (SizeAttribute.IsValid() && SizeAttribute.IsValidIndex(BoneIndex)) ? SizeAttribute[BoneIndex] : 0.0f;
						HoverString = FString::Printf(TEXT("%d: %.2f"), BoneIndex, InspectedValue);
					}
					break;

				default:
					// Invalid inspection attribute
					check(false);
				}
			}
		}	
	}
}


void FGeometryCollectionHistogramItem::SetNormalizedValue(float MinValue, float MaxValue)
{
	if ((MaxValue - MinValue) > KINDA_SMALL_NUMBER)
	{
		NormalizedValue = (InspectedValue - MinValue) / (MaxValue - MinValue);
	} 
	else
	{
		NormalizedValue = 1.0;
	}
}


FGeometryCollectionHistogramItemPtr FGeometryCollectionHistogramItemComponent::GetItemFromBoneIndex(int32 BoneIndex) const
{
	return ItemByBoneIndex.FindRef(BoneIndex);
}

FGeometryCollectionHistogramItemList FGeometryCollectionHistogramItemComponent::RegenerateNodes(int32 LevelView)
{
	// Filter nodes by simulation type
	UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
	TArray<bool> FilterNodeFlags;
	FilterNodeFlags.SetNum(FGeometryCollection::ESimulationTypes::FST_Max);
	FilterNodeFlags[FGeometryCollection::ESimulationTypes::FST_None] = HistogramSettings->bShowEmbedded;
	FilterNodeFlags[FGeometryCollection::ESimulationTypes::FST_Rigid] = HistogramSettings->bShowRigids;
	FilterNodeFlags[FGeometryCollection::ESimulationTypes::FST_Clustered] = HistogramSettings->bShowClusters;
	
	// Collect the inspected attribute 
	FGeometryCollectionHistogramItemList NodesList;
	
	if (Component.IsValid() && Component->GetRestCollection())
	{
		FGeometryCollection* Collection = Component->GetRestCollection()->GetGeometryCollection().Get();

		ItemByBoneIndex.Empty();

		if (Collection)
		{
			int32 NumElements = Collection->NumElements(FGeometryCollection::TransformGroup);

			const TManagedArray<int32>& GeometryToTransform = Collection->GetAttribute<int32>("TransformIndex", "Geometry");
			const TManagedArray<FLinearColor>& BoneColor = Collection->GetAttribute<FLinearColor>("BoneColor", "Transform");
		
			// Add a sub item to the histogram for each of the geometry nodes in this GeometryCollection
			for (int32 Index = 0; Index < NumElements; Index++)
			{
				if (FilterNodeFlags[Collection->SimulationType[Index]])
				{
					if (LevelView > -1)
					{
						// Don't display nodes that aren't visible at the currently inspected level.
						if (Collection->HasAttribute("Level", FGeometryCollection::TransformGroup))
						{
							const TManagedArray<int32>& Levels = Collection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
							if (Levels[Index] != LevelView)
							{
								continue;
							}
						}
					}

					TSharedRef<FGeometryCollectionHistogramItem> NewItem = MakeShared<FGeometryCollectionHistogramItem>(Index, AsShared());

					const FLinearColor GeoColor = BoneColor[Index];
					NewItem->SetColor(GeoColor);

					NodesList.Add(NewItem);
					ItemByBoneIndex.Add(Index, NewItem);
				}
			}
		}
	}

	return NodesList;

}

void SGeometryCollectionHistogram::Construct(const FArguments& InArgs)
{
	BoneSelectionChangedDelegate = InArgs._OnBoneSelectionChanged;
	bPerformingSelection = false;

	ChildSlot
	[
		SAssignNew(ListView, SListView<FGeometryCollectionHistogramItemPtr>)
		.ListItemsSource(&LeafNodes)
		.OnSelectionChanged(this, &SGeometryCollectionHistogram::OnSelectionChanged)
		.OnGenerateRow(this, &SGeometryCollectionHistogram::MakeHistogramRowWidget)
	];
}

void SGeometryCollectionHistogram::SetComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents, int32 LevelView)
{
	// Clear the cached Tree ItemSelection without affecting the SelectedBones as 
	// we want to refresh the tree selection using selected bones
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);
	ListView->ClearSelection();
	
	RootNodes.Empty();
	LeafNodes.Empty();

	if (InNewComponents.Num() > 0)
	{
		for (UGeometryCollectionComponent* Component : InNewComponents)
		{
			if (!ensure(Component))
			{
				continue;
			}
			RootNodes.Add(MakeShared<FGeometryCollectionHistogramItemComponent>(Component));
			LeafNodes.Append(RootNodes.Last()->RegenerateNodes(LevelView));
		}

		SetListIndices();

		UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
		InspectAttribute(HistogramSettings->InspectedAttribute);
		NormalizeInspectedValues();
	}
	
	UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
	RefreshView(HistogramSettings->bSorted);
}

void SGeometryCollectionHistogram::InspectAttribute(EInspectedAttributeEnum InspectedAttribute)
{
	for (FGeometryCollectionHistogramItemPtr Node : LeafNodes)
	{
		Node->SetInspectedAttribute(InspectedAttribute);
	}
	NormalizeInspectedValues();

	UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
	RefreshView(HistogramSettings->bSorted);
}

void SGeometryCollectionHistogram::RefreshView(bool bSorted)
{
	
	if (LeafNodes.Num() > 0)
	{
		if (bSorted)
		{
			// Sort nodes by value
			LeafNodes.Sort([](const TSharedPtr<FGeometryCollectionHistogramItem> A, const TSharedPtr<FGeometryCollectionHistogramItem> B)
				{
					return A->GetInspectedValue() > B->GetInspectedValue(); 
				});

		}
		else 
		{
			// Sort nodes by value
			LeafNodes.Sort([](const TSharedPtr<FGeometryCollectionHistogramItem> A, const TSharedPtr<FGeometryCollectionHistogramItem> B)
				{
					return A->GetListIndex() > B->GetListIndex();
				});
		}	
	}

	ListView->RebuildList();
}

void SGeometryCollectionHistogram::RegenerateNodes(int32 LevelView)
{
	LeafNodes.Empty();
	for (TSharedPtr<FGeometryCollectionHistogramItemComponent> Root : RootNodes)
	{
		LeafNodes.Append(Root->RegenerateNodes(LevelView));
	}
	SetListIndices();

	UHistogramSettings* HistogramSettings = GetMutableDefault<UHistogramSettings>();
	InspectAttribute(HistogramSettings->InspectedAttribute);
	NormalizeInspectedValues();
	RefreshView(HistogramSettings->bSorted);
}


void SGeometryCollectionHistogram::OnSelectionChanged(FGeometryCollectionHistogramItemPtr Item, ESelectInfo::Type SelectInfo)
{
	if (!bPerformingSelection && BoneSelectionChangedDelegate.IsBound())
	{
		TMap<UGeometryCollectionComponent*, TArray<int32>> ComponentToBoneSelectionMap;

		ComponentToBoneSelectionMap.Reserve(RootNodes.Num());

		// Create an entry for each component in the tree.  If the component has no selected bones then we return an empty array to signal that the selection should be cleared
		for (const TSharedPtr<FGeometryCollectionHistogramItemComponent>& Root : RootNodes)
		{
			ComponentToBoneSelectionMap.Add(Root->GetComponent(), TArray<int32>());
		}

		if (!Item.IsValid())
		{
			ListView->ClearSelection();
		}

		FGeometryCollectionHistogramItemList SelectedItems;
		ListView->GetSelectedItems(SelectedItems);

		FScopedTransaction Transaction(FractureTransactionContexts::SelectBoneContext, LOCTEXT("SelectGeometryCollectionBoneTransaction", "Select Bone"), Item.IsValid() ? Item->GetComponent() : nullptr);
		for (TSharedPtr<FGeometryCollectionHistogramItem> SelectedItem : SelectedItems)
		{
			if (SelectedItem->GetBoneIndex() != INDEX_NONE)
			{
				TArray<int32>& SelectedBones = ComponentToBoneSelectionMap.FindChecked(SelectedItem->GetComponent());
				SelectedBones.Add(SelectedItem->GetBoneIndex());
				SelectedItem->GetComponent()->Modify();
			}
		}

		// Fire off the delegate for each component
		for (TPair<UGeometryCollectionComponent*, TArray<int32>>& SelectionPair : ComponentToBoneSelectionMap)
		{
			BoneSelectionChangedDelegate.Execute(SelectionPair.Key, SelectionPair.Value);
		}
	}
	
}


void SGeometryCollectionHistogram::NormalizeInspectedValues()
{
	if (LeafNodes.Num())
	{
		// Sort nodes by value
		LeafNodes.Sort([](const TSharedPtr<FGeometryCollectionHistogramItem> A, const TSharedPtr<FGeometryCollectionHistogramItem> B)
			{
				return A->GetInspectedValue() > B->GetInspectedValue();
			});

		// Normalize
		float MaxInspectedValue = LeafNodes[0]->GetInspectedValue();
		float MinInspectedValue = LeafNodes.Last()->GetInspectedValue();

		// We allow a little extra room at the bottom so that bars don't drop to zero length
		MinInspectedValue -= (MaxInspectedValue - MinInspectedValue) * 0.025;

		for (TSharedPtr<FGeometryCollectionHistogramItem> LeafNode : LeafNodes)
		{
			LeafNode->SetNormalizedValue(MinInspectedValue, MaxInspectedValue);
		}
	}
}

void SGeometryCollectionHistogram::SetListIndices()
{
	int32 NodeCount = LeafNodes.Num();
	for (int32 Index = 0; Index < NodeCount; ++Index)
	{
		LeafNodes[Index]->SetListIndex(Index);
	}
}


void SGeometryCollectionHistogram::SetBoneSelection(UGeometryCollectionComponent* RootComponent, const TArray<int32>& InSelection, bool bClearCurrentSelection, int32 FocusBoneIdx)
{
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);

	if (bClearCurrentSelection)
	{
		ListView->ClearSelection();
	}

	bool bFirstSelection = true;

	for (TSharedPtr<FGeometryCollectionHistogramItemComponent> RootNode : RootNodes)
	{
		if (RootNode->GetComponent() == RootComponent)
		{
			for (int32 BoneIndex : InSelection)
			{
				FGeometryCollectionHistogramItemPtr Item = RootNode->GetItemFromBoneIndex(BoneIndex);
				if (Item.IsValid())
				{
					if (bFirstSelection && BoneIndex == FocusBoneIdx)
					{
						ListView->RequestScrollIntoView(Item);
						bFirstSelection = false;
					}
					ListView->SetItemSelection(Item, true);
					ListView->SetItemHighlighted(Item, true);
				}
			}
			break;
		}
	}
	ListView->RequestListRefresh();

}

TSharedRef<ITableRow> SGeometryCollectionHistogram::MakeHistogramRowWidget(FGeometryCollectionHistogramItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeHistogramRowWidget(InOwnerTable);
}


#undef LOCTEXT_NAMESPACE

