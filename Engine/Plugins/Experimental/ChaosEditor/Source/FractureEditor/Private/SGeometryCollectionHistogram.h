// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Math/Color.h"
#include "Widgets/Views/SListView.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"

#include "SGeometryCollectionHistogram.generated.h"



struct FLinearColor;
class FGeometryCollectionHistogramItem;
class ITableRow;
class STableViewBase;

using FGeometryCollectionHistogramItemPtr = TSharedPtr<FGeometryCollectionHistogramItem>;
using FGeometryCollectionHistogramItemList = TArray<TSharedPtr<FGeometryCollectionHistogramItem>>;


UENUM(BlueprintType)
enum class EInspectedAttributeEnum : uint8
{
	Volume = 0					UMETA(DisplayName = "Volume"),
	Level = 1					UMETA(DisplayName = "Level"),
	InitialDynamicState = 3		UMETA(DisplayName = "InitialDynamicState"),
	Size = 4					UMETA(DisplayName = "RelativeSize")
};

/** Settings for Histogram configuration. **/
UCLASS()
class UHistogramSettings : public UObject
{

	GENERATED_BODY()
public:
	UHistogramSettings(const FObjectInitializer& ObjInit);

	/** What attribute are we inspecting? */
	UPROPERTY(EditAnywhere, Category = HistogramSettings, meta = (DisplayName = "Inspected Attribute"))
	EInspectedAttributeEnum InspectedAttribute;

	/** Sort the values? */
	UPROPERTY(EditAnywhere, Category = HistogramSettings, meta = (DisplayName = "Sort Values"))
	bool bSorted;

	/** Show clusters? */
	UPROPERTY(EditAnywhere, Category = HistogramSettings, meta = (DisplayName = "Show Clusters"))
	bool bShowClusters=true;

	/** Show rigids? */
	UPROPERTY(EditAnywhere, Category = HistogramSettings, meta = (DisplayName = "Show Rigids"))
	bool bShowRigids=true;

	/** Show embedded geometry? */
	UPROPERTY(EditAnywhere, Category = HistogramSettings, meta = (DisplayName = "Show Embedded Geometry"))
	bool bShowEmbedded=true;

};

class FGeometryCollectionHistogramItemComponent : public TSharedFromThis<FGeometryCollectionHistogramItemComponent>
{
public:
	FGeometryCollectionHistogramItemComponent(UGeometryCollectionComponent* InComponent)
		: Component(InComponent)
	{
	}

	UGeometryCollectionComponent* GetComponent() const { return Component.IsValid() ? Component.Get() : nullptr; }

	FGeometryCollectionHistogramItemPtr GetItemFromBoneIndex(int32 BoneIndex) const;
	
	FGeometryCollectionHistogramItemList RegenerateNodes(int32 LevelView);

private:
	TWeakObjectPtr<UGeometryCollectionComponent> Component;
	TMap<int32, FGeometryCollectionHistogramItemPtr> ItemByBoneIndex;
};



class FGeometryCollectionHistogramItem : public TSharedFromThis<FGeometryCollectionHistogramItem>
{
public:
	FGeometryCollectionHistogramItem(const int32 InBoneIndex, const TSharedPtr<FGeometryCollectionHistogramItemComponent> InParentComponentItem)
		: BoneIndex(InBoneIndex)
		, ParentComponentItem(InParentComponentItem)
		, ListIndex(0)
		, NodeColor(FLinearColor::Black)
		, NormalizedValue(0.0)
		, InspectedValue(0.0)
	{}

	TSharedRef<ITableRow> MakeHistogramRowWidget(const TSharedRef<STableViewBase>& InOwnerTable);
	
	UGeometryCollectionComponent* GetComponent() const { return ParentComponentItem->GetComponent(); }

	float GetInspectedValue() const { return InspectedValue; }
	void SetColor(const FLinearColor& InColor) { NodeColor = InColor; }
	void SetInspectedAttribute(EInspectedAttributeEnum InspectedAttribute);
	void SetNormalizedValue(float MinValue, float MaxValue);

	int32 GetBoneIndex() const { return BoneIndex; }
	void SetListIndex(int32 InListIndex) { ListIndex = InListIndex; }
	int32 GetListIndex() const { return ListIndex; }

private:
	const int32 BoneIndex;
	const TSharedPtr<FGeometryCollectionHistogramItemComponent> ParentComponentItem;
	
	int32 ListIndex;
	FLinearColor NodeColor; 
	float NormalizedValue;
	float InspectedValue;
	FString HoverString;
};



class SGeometryCollectionHistogram : public SCompoundWidget
{
	DECLARE_DELEGATE_TwoParams(FOnBoneSelectionChanged, UGeometryCollectionComponent*, TArray<int32>&);

	SLATE_BEGIN_ARGS(SGeometryCollectionHistogram)
	{}

	SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	TSharedRef<ITableRow> MakeHistogramRowWidget(FGeometryCollectionHistogramItemPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	void SetComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents, int32 LevelView);
	void SetBoneSelection(UGeometryCollectionComponent* RootComponent, const TArray<int32>& InSelection, bool bClearCurrentSelection, int32 FocusBoneIdx = -1);
	void InspectAttribute(EInspectedAttributeEnum InspectedAttribute);
	void RefreshView(bool bSorted);
	void RegenerateNodes(int32 LevelView);
	void ClearSelection() { ListView->ClearSelection(); }
	bool IsSelected() { return ListView->GetNumItemsSelected() > 0; }

private:
	void OnSelectionChanged(FGeometryCollectionHistogramItemPtr Item, ESelectInfo::Type SelectInfo);

	void NormalizeInspectedValues();
	void SetListIndices();

private:
	TSharedPtr<SListView<FGeometryCollectionHistogramItemPtr>> ListView;
	TArray<TSharedPtr<FGeometryCollectionHistogramItemComponent>> RootNodes;
	TArray<FGeometryCollectionHistogramItemPtr> LeafNodes;
	
	FOnBoneSelectionChanged BoneSelectionChangedDelegate;
	bool bPerformingSelection;
};
