// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCollectionOutliner.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "PropertyInfoViewStyle.h"

#include "LevelEditor.h"
#include "ToolMenus.h"
#include "Modules/ModuleManager.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "FractureEditorMode.h"
#include "ScopedTransaction.h"
#include "FractureSettings.h"
#include "FractureToolProperties.h"
#include "GeometryCollectionOutlinerDragDrop.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Dataflow/DataflowSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SGeometryCollectionOutliner)

#define LOCTEXT_NAMESPACE "ChaosEditor"

static FText GetTextFromInitialDynamicState(int32 InitialDynamicState)
{
	switch (InitialDynamicState)
	{
	case 0: return LOCTEXT("GCOutliner_InitialState_NoOverride_Text", "-");
	case 1: return LOCTEXT("GCOutliner_InitialState_Sleeping_Text", "Sleeping");
	case 2: return LOCTEXT("GCOutliner_InitialState_Kinematic_Text", "Kinematic");
	case 3: return LOCTEXT("GCOutliner_InitialState_Static_Text", "Static");
	default:
		return FText();
	}
}

static FText GetTextFromAnchored(bool bAnchored)
{
	if (bAnchored)
	{
		return LOCTEXT("GCOutliner_Anchored_Yes_Text", "Yes");
	}
	return LOCTEXT("GCOutliner_Anchored_No_Text", "-");
}

void FGeometryCollectionTreeItem::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = InDragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		if (Operation->IsOfType<FGeometryCollectionBoneDragDrop>())
		{
			TSharedPtr<FGeometryCollectionBoneDragDrop> GeometryCollectionBoneOp = InDragDropEvent.GetOperationAs<FGeometryCollectionBoneDragDrop>();
			const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			GeometryCollectionBoneOp->SetToolTip(FText(), Icon);
		}
	}
}

void FGeometryCollectionTreeItem::GenerateContextMenu(UToolMenu* Menu, SGeometryCollectionOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<SGeometryCollectionOutliner>(Outliner.AsShared());

	FToolMenuSection& StateSection = Menu->AddSection("State");
	StateSection.AddSubMenu("FractureToolSetInitialDynamicStateMenu", NSLOCTEXT("Fracture", "FractureToolSetInitialDynamicStateMenu", "Initial Dynamic State"), FText(),
		FNewToolMenuDelegate::CreateLambda([&Outliner](UToolMenu* Menu)
		{
			constexpr int32 MenuEntryNamesCount = 3;
			const TPair<FName, EObjectStateTypeEnum> MenuEntryNameState[MenuEntryNamesCount] =
			{
				{"NoOverride",EObjectStateTypeEnum::Chaos_NONE},
				// Note: Sleeping state intentionally skipped here, as it's not a valid initial state
				{"Kinematic",EObjectStateTypeEnum::Chaos_Object_Kinematic},
				{"Static",EObjectStateTypeEnum::Chaos_Object_Static}
			};

			FToolMenuSection& StateSection = Menu->AddSection("State");
			for (int32 Index = 0; Index < MenuEntryNamesCount; ++Index)
			{
				int32 State = (int32)MenuEntryNameState[Index].Value;
				StateSection.AddMenuEntry(MenuEntryNameState[Index].Key, GetTextFromInitialDynamicState(State), FText(), FSlateIcon(),
					FUIAction(FExecuteAction::CreateRaw(&Outliner, &SGeometryCollectionOutliner::SetInitialDynamicState, State)));
			}
		}));

	FToolMenuSection& AnchoredSection = Menu->AddSection("Anchored");
	StateSection.AddSubMenu("FractureToolSetAnchoredMenu", NSLOCTEXT("Fracture", "FractureToolSetAnchoredMenu", "Anchored"), FText(),
		FNewToolMenuDelegate::CreateLambda([&Outliner](UToolMenu* Menu)
		{
			FToolMenuSection& AnchoredSection = Menu->AddSection("Anchored");
			{
				AnchoredSection.AddMenuEntry("Yes", LOCTEXT("GCOutliner_Anchored_Yes_ContextMenuText", "Yes"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SGeometryCollectionOutliner::SetAnchored, true)));
				AnchoredSection.AddMenuEntry("No", LOCTEXT("GCOutliner_Anchored_No_ContextMenuText", "No"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SGeometryCollectionOutliner::SetAnchored, false)));
			}
		}));
}

FColor FGeometryCollectionTreeItem::GetColorPerDepth(uint32 Depth)
{
	const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

	const uint32 ColorPerDepthCount = DataflowSettings->TransformLevelColors.LevelColors.Num();
	return DataflowSettings->TransformLevelColors.LevelColors[Depth % ColorPerDepthCount].ToFColor(true);
}

void FGeometryCollectionTreeItemBone::OnDragEnter(FDragDropEvent const& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = InDragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		if (Operation->IsOfType<FGeometryCollectionBoneDragDrop>())
		{
			TSharedPtr<FGeometryCollectionBoneDragDrop> GeometryCollectionBoneOp = InDragDropEvent.GetOperationAs<FGeometryCollectionBoneDragDrop>();
			
			UGeometryCollectionComponent* SourceComponent = GetComponent();
			FGeometryCollectionEdit GeometryCollectionEdit = SourceComponent->EditRestCollection();
			if (UGeometryCollection* GeometryCollectionObject = GeometryCollectionEdit.GetRestCollection())
			{
				FGeometryCollection* GeometryCollection = GeometryCollectionObject->GetGeometryCollection().Get();
				FText HoverText;
				bool bValid = GeometryCollectionBoneOp->ValidateDrop(GeometryCollection, BoneIndex, HoverText);
				const FSlateBrush* Icon = bValid ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				GeometryCollectionBoneOp->SetToolTip(HoverText, Icon);
			}
		}
	}
}

FReply FGeometryCollectionTreeItemBone::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		UGeometryCollectionComponent* SourceComponent = GetComponent();

		FGeometryCollectionEdit GeometryCollectionEdit = SourceComponent->EditRestCollection();
		if (UGeometryCollection* GeometryCollectionObject = GeometryCollectionEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection,ESPMode::ThreadSafe> GeometryCollection = GeometryCollectionObject->GetGeometryCollection();
			TArray<int32> SelectedBones = SourceComponent->GetSelectedBones();

			return FReply::Handled().BeginDragDrop(FGeometryCollectionBoneDragDrop::New(GeometryCollection, SelectedBones));
		}		
	}

	return FReply::Unhandled();
}

FReply FGeometryCollectionTreeItemBone::OnDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid()) 
	{
		return FReply::Unhandled();
	}
	
	if (Operation->IsOfType<FGeometryCollectionBoneDragDrop>())
	{
		TSharedPtr<FGeometryCollectionBoneDragDrop> GeometryCollectionBoneDragDrop = StaticCastSharedPtr<FGeometryCollectionBoneDragDrop>(Operation);

		UGeometryCollectionComponent* SourceComponent = GetComponent();

		FGeometryCollectionEdit GeometryCollectionEdit = SourceComponent->EditRestCollection();
		if (UGeometryCollection* GeometryCollectionObject = GeometryCollectionEdit.GetRestCollection())
		{
			FGeometryCollection* GeometryCollection = GeometryCollectionObject->GetGeometryCollection().Get();
			if (GeometryCollectionBoneDragDrop->ReparentBones(GeometryCollection, BoneIndex))
			{
				ParentComponentItem->RegenerateChildren();
				ParentComponentItem->RequestTreeRefresh();
				ParentComponentItem->ExpandAll();
			}
		}		
	}
	
	return FReply::Unhandled();
}

UOutlinerSettings::UOutlinerSettings(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, ColorByLevel(true)
	, ColumnMode(EOutlinerColumnMode::State)
{}

TSharedRef<SWidget> SGeometryCollectionOutlinerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	// This can happen because sometimes slate retains old items until the next tick, and keeps calling callbacks on them until then
	if (!Item->IsValidBone())
	{
		return Item->MakeEmptyColumnWidget();
	}

	if (ColumnName == SGeometryCollectionOutlinerColumnID::BoneIndex)
	{
		const TSharedPtr<SWidget> NameWidget = Item->MakeBoneIndexColumnWidget();
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
					.ShouldDrawWires(true)
				]
			+ SHorizontalBox::Slot()
				[
					NameWidget.ToSharedRef()
				];
	}
	if (ColumnName == SGeometryCollectionOutlinerColumnID::BoneName)
		return Item->MakeBoneNameColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::RelativeSize)
		return Item->MakeRelativeSizeColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::Volume)
		return Item->MakeVolumeColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::InitialState)
		return Item->MakeInitialStateColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::Anchored)
		return Item->MakeAnchoredColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::Damage)
		return Item->MakeDamagesColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::DamageThreshold)
		return Item->MakeDamageThresholdColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::Broken)
		return Item->MakeBrokenColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::PostBreakTime)
		return Item->MakePostBreakTimeColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::RemovalTime)
		return Item->MakeRemovalTimeColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::ImportedCollisions)
		return Item->MakeImportedCollisionsColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::ConvexCount)
		return Item->MakeConvexCountColumnWidget();
	if (ColumnName == SGeometryCollectionOutlinerColumnID::TriangleCount)
	{
		return Item->MakeTriangleCountColumnWidget();
	}
	if (ColumnName == SGeometryCollectionOutlinerColumnID::VertexCount)
	{
		return Item->MakeVertexCountColumnWidget();
	}

	return Item->MakeEmptyColumnWidget();
}

void SGeometryCollectionOutliner::Construct(const FArguments& InArgs)
{
	BoneSelectionChangedDelegate = InArgs._OnBoneSelectionChanged;
	bPerformingSelection = false;

	HeaderRowWidget =
		SNew( SHeaderRow )
			.Visibility( EVisibility::Visible );
	RegenerateHeader();
	
	ChildSlot
	[
		SAssignNew(TreeView, STreeView<FGeometryCollectionTreeItemPtr>)
		.TreeItemsSource(reinterpret_cast<FGeometryCollectionTreeItemList*>(&RootNodes))
		.OnSelectionChanged(this, &SGeometryCollectionOutliner::OnSelectionChanged)
		.OnGenerateRow(this, &SGeometryCollectionOutliner::MakeTreeRowWidget)
		.OnGetChildren(this, &SGeometryCollectionOutliner::OnGetChildren)
		.OnContextMenuOpening(this, &SGeometryCollectionOutliner::OnOpenContextMenu)
		.AllowInvisibleItemSelection(true)
		.ShouldStackHierarchyHeaders(true)
		.OnGeneratePinnedRow(this, &SGeometryCollectionOutliner::OnGeneratePinnedRowWidget, true)
		.HighlightParentNodesForSelection(true)
		.OnSetExpansionRecursive(this, &SGeometryCollectionOutliner::ExpandRecursive)
		.HeaderRow(HeaderRowWidget)
	];
}

void SGeometryCollectionOutliner::RegenerateHeader()
{
	constexpr int32 CustomFillWidth = 2.0f;

	HeaderRowWidget->ClearColumns();

	RegenerateRootData();

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::BoneIndex)
		.DefaultLabel(LOCTEXT("GCOutliner_Column_BoneIndex", "Index"))
		.FillWidth(2.0f)
	);
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::BoneName)
		.DefaultLabel(LOCTEXT("GCOutliner_Column_BoneName", "Name"))
		.FillWidth(2.0f)
	);

	// then add the right customn one based on the selection
	UOutlinerSettings* OutlinerSettings = GetMutableDefault<UOutlinerSettings>();
	switch (OutlinerSettings->ColumnMode)
	{
	case EOutlinerColumnMode::State:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::InitialState)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_InitialState", "Initial State"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_InitialState_ToolTip", "Initial state override"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::Anchored)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_Anchored", "Anchored"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_Anchored_ToolTip", "Anchored"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		break;

	case EOutlinerColumnMode::Damage:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::Damage)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_MaxDamage", "Max damage"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_MaxDamage_ToolTip", "Maximum amount of damage recorded ( through collision )"))
	 			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
	 			.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::DamageThreshold)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_DamageThreshold", "Damage Threshold"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_DamageThreshold_ToolTip", "Current damage threshold ( equivalent to internal strain )"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::Broken)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_Broken", "Broken"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_Broken_ToolTip", "Whether the piece has broken off"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(1.0)
		);
		break;
	case EOutlinerColumnMode::Removal:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::PostBreakTime)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_PostBreakTime", "Post Break Time"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_PostBreakTime_ToolTip", "Min/Max time after break until removal starts"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::RemovalTime)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_RemovalTime", "Removal Time"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_RemovalTime_ToolTip", "Min/Max time for removal"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		break;
		
	case EOutlinerColumnMode::Collision:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::ConvexCount)
			.DefaultLabel(LOCTEXT("GCOutliner_Column_ConvexCount", "Convex Count"))
			.DefaultTooltip(LOCTEXT("GCOutliner_Column_ConvexCount_ToolTip", "Number of convex collisions)"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Right)
			.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::ImportedCollisions)
				.DefaultLabel(LOCTEXT("GCOutliner_Column_ImportedCollisions", "Imported Collisions"))
				.DefaultTooltip(LOCTEXT("GCOutliner_Column_ImportedCollisions_ToolTip", "Status of imported Collision [available, used, nothing])"))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.FillWidth(CustomFillWidth)
		);
		break;

	case EOutlinerColumnMode::Size:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::RelativeSize)
			.DefaultLabel(LOCTEXT("GCOutliner_Column_RelativeSize", "Relative Size"))
			.DefaultTooltip(LOCTEXT("GCOutliner_Column_RelativeSize_ToolTip", "Relative size ( Used for size specific data )"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::Volume)
			.DefaultLabel(LOCTEXT("GCOutliner_Column_Volume", "Size"))
			.DefaultTooltip(LOCTEXT("GCOutliner_Column_Volume_ToolTip", "Cube root of bone volume (in cm)"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FillWidth(CustomFillWidth)
		);
		break;

	case EOutlinerColumnMode::Geometry:
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::TriangleCount)
			.DefaultLabel(LOCTEXT("GCOutliner_Column_TriangleCount", "Triangle Count"))
			.DefaultTooltip(LOCTEXT("GCOutliner_Column_TriangleCount_ToolTip", "Number of Triangles"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FillWidth(CustomFillWidth)
		);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SGeometryCollectionOutlinerColumnID::VertexCount)
			.DefaultLabel(LOCTEXT("GCOutliner_Column_VertexCount", "Vertex Count"))
			.DefaultTooltip(LOCTEXT("GCOutliner_Column_VertexCount_ToolTip", "Number of Vertices"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FillWidth(CustomFillWidth)
		);
		break;

	}
}

void SGeometryCollectionOutliner::RegenerateItems()
{
	RegenerateRootData();
	TreeView->RebuildList();
}

void SGeometryCollectionOutliner::RegenerateRootData()
{
	for (TSharedPtr<FGeometryCollectionTreeItemComponent> RootNode : RootNodes)
	{
		if (RootNode)
		{
			RootNode->GenerateDataCollection();
		}
	}
}

TSharedRef<ITableRow> SGeometryCollectionOutliner::MakeTreeRowWidget(FGeometryCollectionTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeTreeRowWidget(InOwnerTable);
}

TSharedRef<ITableRow> SGeometryCollectionOutliner::OnGeneratePinnedRowWidget(FGeometryCollectionTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned)
{
	return InItem->MakeTreeRowWidget(InOwnerTable, true);
}

void SGeometryCollectionOutliner::OnGetChildren(FGeometryCollectionTreeItemPtr InItem, TArray<FGeometryCollectionTreeItemPtr>& OutChildren)
{
	InItem->GetChildren(OutChildren);
}

TSharedPtr<SWidget> SGeometryCollectionOutliner::OnOpenContextMenu()
{
	FGeometryCollectionTreeItemList SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num())
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		static const FName MenuName = "SGeometryCollectionOutliner.GeometryCollectionOutlinerContextMenu";
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName);
		}

		// Build up the menu for a selection
		FToolMenuContext Context;
		UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
		SelectedItems[0]->GenerateContextMenu(Menu, *this);
		return ToolMenus->GenerateWidget(Menu);
	}

	return TSharedPtr<SWidget>();
}

void SGeometryCollectionOutliner::UpdateGeometryCollection()
{
	TreeView->RequestTreeRefresh();
	ExpandAll();
}

void SGeometryCollectionOutliner::SetComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents)
{
	// Clear the cached Tree ItemSelection without affecting the SelectedBones as 
	// we want to refresh the tree selection using selected bones
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);
	TreeView->ClearSelection();

	// explicitly mark the root nodes as invalid before emptying, so we know we can safely ignore them in case slate still triggers callbacks for them (they will not be deleted until the tree view refresh, on tick)
	for (TSharedPtr<FGeometryCollectionTreeItemComponent>& RootNode : RootNodes)
	{
		if (RootNode)
		{
			RootNode->Invalidate();
		}
	}
	RootNodes.Empty();

	for (UGeometryCollectionComponent* Component : InNewComponents)
	{
		if (Component->GetRestCollection() && IsValidChecked(Component->GetRestCollection()))
		{
			RootNodes.Add(MakeShared<FGeometryCollectionTreeItemComponent>(Component, TreeView));
			TArray<int32> SelectedBones = Component->GetSelectedBones();
			SetBoneSelection(Component, SelectedBones, false);
		}
	}

	TreeView->RequestTreeRefresh();
	ExpandAll();
}

void SGeometryCollectionOutliner::ExpandAll()
{
	for (TSharedPtr<FGeometryCollectionTreeItemComponent> ItemPtr : RootNodes)
	{
		ItemPtr->ExpandAll();
	}
}

void SGeometryCollectionOutliner::ExpandRecursive(TSharedPtr<FGeometryCollectionTreeItem> ItemPtr, bool bInExpansionState) const
{
	TreeView->SetItemExpansion(ItemPtr, bInExpansionState);

	FGeometryCollectionTreeItemList ItemChildren;
	ItemPtr->GetChildren(ItemChildren);
	for (auto& Child : ItemChildren)
	{
		ExpandRecursive(Child, bInExpansionState);
	}
}

void SGeometryCollectionOutliner::SetHistogramSelection(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones)
{
	// Find the matching component
	for (TSharedPtr<FGeometryCollectionTreeItemComponent> RootNode : RootNodes)
	{
		if (RootNode->GetComponent() == RootComponent)
		{
			// Copy the histogram selection.
			RootNode->SetHistogramSelection(SelectedBones);
			RootNode->RegenerateChildren();
			TreeView->RequestTreeRefresh();
			ExpandAll();
			return;
		}
	}
}

int32 SGeometryCollectionOutliner::GetBoneSelectionCount() const
{
	return TreeView->GetSelectedItems().Num();
}

int32 SGeometryCollectionOutliner::GetBoneDisplayedCount() const
{
	return TreeView->GetNumGeneratedChildren();
}

void SGeometryCollectionOutliner::SetBoneSelection(UGeometryCollectionComponent* RootComponent, const TArray<int32>& InSelection, bool bClearCurrentSelection, int32 FocusBoneIdx)
{
	TGuardValue<bool> ExternalSelectionGuard(bPerformingSelection, true);

	if (bClearCurrentSelection)
	{
		TreeView->ClearSelection();
	}

	bool bFirstSelection = true;

	FGeometryCollectionTreeItemList NewSelection;
	for(auto& RootNode : RootNodes)
	{
		if (RootNode->GetComponent() == RootComponent)
		{
			for(int32 BoneIndex : InSelection)
			{
				FGeometryCollectionTreeItemPtr Item = RootNode->GetItemFromBoneIndex(BoneIndex);
				if (ensure(Item.IsValid()))
				{
					if (bFirstSelection && FocusBoneIdx == BoneIndex)
					{
						TreeView->RequestScrollIntoView(Item);
						bFirstSelection = false;
					}
					NewSelection.Add(Item);
				}
			}
			break;
		}
	}
	TreeView->SetItemSelection(NewSelection, true);
}

void SGeometryCollectionOutliner::OnSelectionChanged(FGeometryCollectionTreeItemPtr Item, ESelectInfo::Type SelectInfo)
{
	// object may just being deleted, we need to bail if that's the case to avoid a crash accessing teh component
	if (!Item || !IsValid(Item->GetComponent()))
	{
		return;
	}

	if(!bPerformingSelection && BoneSelectionChangedDelegate.IsBound())
	{
		TMap<UGeometryCollectionComponent*, TArray<int32>> ComponentToBoneSelectionMap;

		ComponentToBoneSelectionMap.Reserve(RootNodes.Num());

		// Create an entry for each component in the tree.  If the component has no selected bones then we return an empty array to signal that the selection should be cleared
		for (auto& Root : RootNodes)
		{
			ComponentToBoneSelectionMap.Add(Root->GetComponent(), TArray<int32>());
		}

		if (Item == nullptr)
		{
			TreeView->ClearSelection();
		}

		FGeometryCollectionTreeItemList SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		FScopedTransaction Transaction(FractureTransactionContexts::SelectBoneContext, LOCTEXT("SelectGeometryCollectionBoneTransaction", "Select Bone"), Item != nullptr ? Item->GetComponent() : nullptr);
		for(auto& SelectedItem : SelectedItems)
		{
			if (SelectedItem->GetBoneIndex() != INDEX_NONE)
			{
				TArray<int32>& SelectedBones = ComponentToBoneSelectionMap.FindChecked(SelectedItem->GetComponent());
				SelectedBones.Add(SelectedItem->GetBoneIndex());
				SelectedItem->GetComponent()->Modify();
			}
		}
		// Fire off the delegate for each component
		for (auto& SelectionPair : ComponentToBoneSelectionMap)
		{
			BoneSelectionChangedDelegate.Execute(SelectionPair.Key, SelectionPair.Value);
		}
	}
}

void SGeometryCollectionOutliner::SetInitialDynamicState(int32 InDynamicState)
{
	UFractureToolSetInitialDynamicState::SetSelectedInitialDynamicState(InDynamicState);
	RegenerateItems();
}

void SGeometryCollectionOutliner::SetAnchored(bool bAnchored)
{
	// todo : eventually move this to a more central place ( tools ?)  
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	UFractureToolSetInitialDynamicState::GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		FGeometryCollectionEdit GCEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::RestPhysics, true /*bShapeIsUnchanged*/);
		if (UGeometryCollection* GCObject = GCEdit.GetRestCollection())
		{
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GCObject->GetGeometryCollection();
			if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
			{
				Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(*GeometryCollection);
				if (!AnchoringFacade.HasAnchoredAttribute())
				{
					AnchoringFacade.AddAnchoredAttribute();
				}
				AnchoringFacade.SetAnchored(GeometryCollectionComponent->GetSelectedBones(), bAnchored);
			}
		}
	}
	RegenerateItems();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

// for now use the transform group as this simplify the attribute copy 
static const FName DataCollectionGroup = FGeometryCollection::TransformGroup;

FGeometryCollectionItemDataFacade::FGeometryCollectionItemDataFacade(FManagedArrayCollection& InCollection)
	: DataCollection(InCollection)
	, BoneNameAttribute(InCollection, "BoneName", DataCollectionGroup)
	, LevelAttribute(InCollection, "Level", DataCollectionGroup)
	, VisibleAttribute(InCollection, "Visible", DataCollectionGroup)
	, InitialStateAttribute(InCollection, "InitialState", DataCollectionGroup)
	, RelativeSizeAttribute(InCollection, "RelativeSize", DataCollectionGroup)
	, VolumetricUnitAttribute(InCollection, "VolumetricUnit", DataCollectionGroup)
	, AnchoredAttribute(InCollection, "Anchored", DataCollectionGroup)
	, DamageAttribute(InCollection, "Damage", DataCollectionGroup)
	, DamageThresholdAttribute(InCollection, "DamageThreshold", DataCollectionGroup)
	, BrokenStateAttribute(InCollection, "BrokenState", DataCollectionGroup)
	, RemoveOnBreakAttribute(InCollection, "RemoveOnBreak", DataCollectionGroup)
	, SimulationTypeAttribute(InCollection, "SimulationType", DataCollectionGroup)
	, HasSourceCollisionAttribute(InCollection, "HasSourceCollision", DataCollectionGroup)
	, SourceCollisionUsedAttribute(InCollection, "SourceCollisionUsed", DataCollectionGroup)
	, ConvexCountAttribute(InCollection, "ConvexCount", DataCollectionGroup)
	, TriangleCountAttribute(InCollection, "TriangleCount", DataCollectionGroup)
	, VertexCountAttribute(InCollection, "VertexCount", DataCollectionGroup)
{
}

template <typename T>
static T GetAttributeValue(const TManagedArrayAccessor<T>& Attribute, int32 Index, T Default)
{
	return (Attribute.IsValid()) ? Attribute.Get()[Index] : Default;
}

bool FGeometryCollectionItemDataFacade::IsValidBoneIndex(int32 BoneIndex) const
{
	return BoneIndex >= 0 && BoneIndex < BoneNameAttribute.Num();
}

int32 FGeometryCollectionItemDataFacade::GetBoneCount() const
{
	return BoneNameAttribute.Num();
}

FString FGeometryCollectionItemDataFacade::GetBoneName(int32 Index) const
{
	return GetAttributeValue(BoneNameAttribute, Index, FString());
}

float FGeometryCollectionItemDataFacade::GetRelativeSize(int32 Index) const
{
	return GetAttributeValue(RelativeSizeAttribute, Index, 0.0f);
}

float FGeometryCollectionItemDataFacade::GetVolumetricUnit(int32 Index) const
{
	return GetAttributeValue(VolumetricUnitAttribute, Index, 0.0f);
}
int32 FGeometryCollectionItemDataFacade::GetInitialState(int32 Index) const
{
	return GetAttributeValue(InitialStateAttribute, Index, 0);
}
bool FGeometryCollectionItemDataFacade::IsAnchored(int32 Index) const
{
	return GetAttributeValue(AnchoredAttribute, Index, false);
}
float FGeometryCollectionItemDataFacade::GetDamage(int32 Index) const
{
	return GetAttributeValue(DamageAttribute, Index, 0.0f);
}
float FGeometryCollectionItemDataFacade::GetDamageThreshold(int32 Index) const
{
	return GetAttributeValue(DamageThresholdAttribute, Index, 0.0f);
}
bool FGeometryCollectionItemDataFacade::IsBroken(int32 Index) const
{
	return GetAttributeValue(BrokenStateAttribute, Index, false);
}
FRemoveOnBreakData FGeometryCollectionItemDataFacade::GetRemoveOnBreakData(int32 Index) const
{
	return GetAttributeValue(RemoveOnBreakAttribute, Index, FRemoveOnBreakData::DisabledPackedData);
}
bool FGeometryCollectionItemDataFacade::HasSourceCollision(int32 Index) const
{
	return GetAttributeValue(HasSourceCollisionAttribute, Index, false);
}
bool FGeometryCollectionItemDataFacade::IsSourceCollisionUsed(int32 Index) const
{
	return GetAttributeValue(SourceCollisionUsedAttribute, Index, false);
}

int32 FGeometryCollectionItemDataFacade::GetConvexCount(int32 Index) const
{
	return GetAttributeValue(ConvexCountAttribute, Index, 0);
}

int32 FGeometryCollectionItemDataFacade::GetTriangleCount(int32 Index) const
{
	return GetAttributeValue(TriangleCountAttribute, Index, 0);
}

int32 FGeometryCollectionItemDataFacade::GetVertexCount(int32 Index) const
{
	return GetAttributeValue(VertexCountAttribute, Index, 0);
}

void FGeometryCollectionItemDataFacade::FillFromGeometryCollectionComponent(const UGeometryCollectionComponent& GeometryCollectionComponent, EOutlinerColumnMode ColumnMode)
{
	const UGeometryCollection* RestCollection = GeometryCollectionComponent.GetRestCollection();
	if (RestCollection && IsValidChecked(RestCollection))
	{
		TSharedPtr<const FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = RestCollection->GetGeometryCollection();
		const FGeometryCollection& GeometryCollection = *GeometryCollectionPtr;

		const int32 GCNumElements = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);

		DataCollection.AddGroup(DataCollectionGroup);
		DataCollection.AddElements(GCNumElements, DataCollectionGroup);

		// bones names are part of the collection regardless of the column mode
		TManagedArrayAccessor<FString> GCBoneNameAttribute(GeometryCollection, "BoneName", FGeometryCollection::TransformGroup);
		if (GCBoneNameAttribute.IsValid())
		{
			BoneNameAttribute.Copy(GCBoneNameAttribute);
		}

		// we also need level, simulation type and visibility for item colors
		// @todo : we copuld use the settings selection option to see if we need all of them or compute the color there
		TManagedArrayAccessor<int32> GCLevelAttribute(GeometryCollection, "Level", FGeometryCollection::TransformGroup);
		if (GCLevelAttribute.IsValid())
		{
			LevelAttribute.Copy(GCLevelAttribute);
		}

		const TManagedArrayAccessor<int32> GCSimulationTypeAttribute(GeometryCollection, "SimulationType", FGeometryCollection::TransformGroup);
		if (GCSimulationTypeAttribute.IsValid())
		{
			SimulationTypeAttribute.Copy(GCSimulationTypeAttribute);
		}

		TManagedArray<bool>& Visible = VisibleAttribute.Add();
		for (int32 Index = 0; Index < GCNumElements; Index++)
		{
			Visible[Index] = GeometryCollection.IsVisible(Index);
		}

		if (ColumnMode == EOutlinerColumnMode::State)
		{
			const TManagedArrayAccessor<int32> GCInitialStateAttribute(GeometryCollection, "InitialDynamicState", FGeometryCollection::TransformGroup);
			if (GCInitialStateAttribute.IsValid())
			{
				InitialStateAttribute.Copy(GCInitialStateAttribute);
			}

			const TManagedArrayAccessor<bool> GCAnchoredAttribute(GeometryCollection, "Anchored", FGeometryCollection::TransformGroup);
			if (GCAnchoredAttribute.IsValid())
			{
				AnchoredAttribute.Copy(GCAnchoredAttribute);
			}
		}
		else if (ColumnMode == EOutlinerColumnMode::Size)
		{
			const TManagedArrayAccessor<float> GCRelativeSizeAttribute(GeometryCollection, "Size", FGeometryCollection::TransformGroup);
			if (GCRelativeSizeAttribute.IsValid())
			{
				RelativeSizeAttribute.Copy(GCRelativeSizeAttribute);
			}

			const TManagedArrayAccessor<float> GCVolumeAttribute(GeometryCollection, "Volume", FGeometryCollection::TransformGroup);
			if (GCVolumeAttribute.IsValid())
			{
				VolumetricUnitAttribute.Copy(GCVolumeAttribute);
				// volumetric unit is the cubic root of the volume so it is more meaningful to read and comprehend than volume
				TManagedArray<float>& VolumetricUnit = VolumetricUnitAttribute.Modify();
				for (int32 Index = 0; Index < VolumetricUnit.Num(); Index++)
				{
					VolumetricUnit[Index] = FMath::Pow(VolumetricUnit[Index], 1.0/3.0);
				}
			}
		}
		else if (ColumnMode == EOutlinerColumnMode::Damage)
		{
			if (const FDamageCollector* Collector = GeometryCollectionComponent.GetRunTimeDataCollector())
			{
				TManagedArray<float>& Damage = DamageAttribute.Add();
				TManagedArray<float>& DamageThreshold = DamageThresholdAttribute.Add();
				TManagedArray<bool>& BrokenState = BrokenStateAttribute.Add();
				for (int32 Index = 0; Index < GCNumElements; Index++)
				{
					const FDamageCollector::FDamageData& DamageData = (*Collector)[Index];
					Damage[Index] = DamageData.MaxDamages;
					DamageThreshold[Index] = DamageData.DamageThreshold;
					BrokenState[Index] = DamageData.bIsBroken;
				}
			}
		}
		else if (ColumnMode == EOutlinerColumnMode::Collision)
		{
			const TManagedArrayAccessor<Chaos::FImplicitObjectPtr> GCSourceCollisionAttribute(GeometryCollection, FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
			if (GCSourceCollisionAttribute.IsValid())
			{
				const TManagedArray<Chaos::FImplicitObjectPtr>& GCSourceCollision = GCSourceCollisionAttribute.Get();
				TManagedArray<bool>& HasSourceCollision = HasSourceCollisionAttribute.Add();
				TManagedArray<bool>& SourceCollisionUsed = SourceCollisionUsedAttribute.Add();
				
				for (int32 Index = 0; Index < GCNumElements; Index++)
				{
					HasSourceCollision[Index] = (GCSourceCollision[Index] != nullptr);
					SourceCollisionUsed[Index] = RestCollection->bImportCollisionFromSource;
				}
			}
			
			TManagedArrayAccessor<TSet<int32>> GCTransformToConvexIndicesAttribute(GeometryCollection, "TransformToConvexIndices", FGeometryCollection::TransformGroup);
			TManagedArray<int32>& ConvexCountArray = ConvexCountAttribute.Add();
			ConvexCountAttribute.Fill(0);

			if (GCTransformToConvexIndicesAttribute.IsValid())
			{
				const TManagedArray<TSet<int32>>& GCTransformToConvexIndices = GCTransformToConvexIndicesAttribute.Get();

				Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);
				const TArray<int32> TransformIndices = HierarchyFacade.GetTransformArrayInDepthFirstOrder();

				for (int32 TransformIndex: TransformIndices)
				{
					const int32 ConvexCount = GCTransformToConvexIndices[TransformIndex].Num();
					if (ConvexCount > 0)
					{
						ConvexCountArray[TransformIndex] = ConvexCount;
					}
					// if count == 0 then we have already accumulated the children in the parents, no need to do anything
					// so now just pass it to the direct parent ( we parse the index in a depth first manner ) 
					int32 ParentTransformIndex = GeometryCollection.Parent[TransformIndex];
					if (ParentTransformIndex != INDEX_NONE)
					{		
						// if parent has no convex then it will be a union of the aggregated children
						const int32 ParentConvexCount = GCTransformToConvexIndices[ParentTransformIndex].Num();
						if (ParentConvexCount == 0)
						{
							// negative values means union 
							ConvexCountArray[ParentTransformIndex] -= FMath::Abs(ConvexCountArray[TransformIndex]);
						}
					}
				}
			}			
		}
		else if (ColumnMode == EOutlinerColumnMode::Removal)
		{
			const TManagedArrayAccessor<FVector4f> GCRemoveOnBreakAttribute(GeometryCollection, "RemoveOnBreak", FGeometryCollection::TransformGroup);
			if (GCRemoveOnBreakAttribute.IsValid())
			{
				RemoveOnBreakAttribute.Copy(GCRemoveOnBreakAttribute);
			}
		}
		else if (ColumnMode == EOutlinerColumnMode::Geometry)
		{
			const TManagedArrayAccessor<int32> GCTransformToGeometryIndexAttribute(GeometryCollection, "TransformToGeometryIndex", FGeometryCollection::TransformGroup);
			const TManagedArrayAccessor<int32> GCGeoVertexCountAttribute(GeometryCollection, "VertexCount", FGeometryCollection::GeometryGroup);
			const TManagedArrayAccessor<int32> GCGeoFaceCountAttribute(GeometryCollection, "FaceCount", FGeometryCollection::GeometryGroup);
			if (GCTransformToGeometryIndexAttribute.IsValid() && GCGeoVertexCountAttribute.IsValid() && GCGeoFaceCountAttribute.IsValid() && GCSimulationTypeAttribute.IsValid())
			{
				TManagedArray<int32>& VertexCountArray = VertexCountAttribute.Add();
				TManagedArray<int32>& TriangleCountArray = TriangleCountAttribute.Add();

				Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(GeometryCollection);
				const TArray<int32> TransformIndices = HierarchyFacade.GetTransformArrayInDepthFirstOrder();

				for (int32 TransformIndex : TransformIndices)
				{
					int32 GeometryIndex = GCTransformToGeometryIndexAttribute[TransformIndex];
					if (GCSimulationTypeAttribute[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid)
					{
						VertexCountArray[TransformIndex] = GCGeoVertexCountAttribute[GeometryIndex];
						TriangleCountArray[TransformIndex] = GCGeoFaceCountAttribute[GeometryIndex];
					}

					int32 ParentTransformIndex = GeometryCollection.Parent[TransformIndex];
					if (ParentTransformIndex != INDEX_NONE)
					{
						VertexCountArray[ParentTransformIndex] -= FMath::Abs(VertexCountArray[TransformIndex]);
						TriangleCountArray[ParentTransformIndex] -= FMath::Abs(TriangleCountArray[TransformIndex]);
					}
				}
			}
			}
	}
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////
TSharedRef<ITableRow> FGeometryCollectionTreeItemComponent::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bNoExtraColumn)
{
	FString ActorName = Component.IsValid()? Component->GetOwner()->GetActorLabel(): FString();
	FString ComponentName = Component.IsValid()? Component->GetClass()->GetFName().ToString() : FString();

	return SNew(STableRow<FGeometryCollectionTreeItemPtr>, InOwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString(ActorName + '.' + ComponentName))
		];
}

void FGeometryCollectionTreeItemComponent::GetChildren(FGeometryCollectionTreeItemList& OutChildren)
{
	OutChildren = ChildItems;
}

FGeometryCollectionTreeItemPtr FGeometryCollectionTreeItemComponent::GetItemFromBoneIndex(int32 BoneIndex) const
{
	return ItemsByBoneIndex.FindRef(BoneIndex);
}

void FGeometryCollectionTreeItemComponent::GetChildrenForBone(FGeometryCollectionTreeItemBone& BoneItem, FGeometryCollectionTreeItemList& OutChildren) const
{
	if (!Component.IsValid())
	{
		return;
	}
	if(const UGeometryCollection* RestCollection = Component->GetRestCollection())
	{
		if (FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get())
		{
			const int32 BoneIndex = BoneItem.GetBoneIndex();
			if (ensure(BoneIndex >= 0 && BoneIndex < Collection->NumElements(FGeometryCollection::TransformGroup)))
			{
				const TManagedArray<TSet<int32>>& Children = Collection->Children;
				for (const int32 ChildIndex : Children[BoneIndex])
				{
					FGeometryCollectionTreeItemPtr ChildPtr = ItemsByBoneIndex.FindRef(ChildIndex);
					if (ChildPtr.IsValid())
					{
						OutChildren.Add(ChildPtr);
					}
				}
			}
		}
	}
}

bool FGeometryCollectionTreeItemComponent::HasChildrenForBone(const FGeometryCollectionTreeItemBone& BoneItem) const
{
	if (!Component.IsValid())
	{
		return false;
	}
	if(const UGeometryCollection* RestCollection = Component->GetRestCollection())
	{
		if (const FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get())
		{
			const int32 BoneIndex = BoneItem.GetBoneIndex();
			if (ensure(BoneIndex >= 0 && BoneIndex < Collection->NumElements(FGeometryCollection::TransformGroup)))
			{
				return Collection->Children[(BoneIndex)].Num() > 0;
			}
		}
	}
	return false;
}

void FGeometryCollectionTreeItemComponent::ExpandAll()
{
	TreeView->SetItemExpansion(AsShared(), true);

	for (const auto& Elem : ItemsByBoneIndex)
	{
	    TreeView->SetItemExpansion(Elem.Value, true);
	}
}

void FGeometryCollectionTreeItemComponent::GenerateDataCollection()
{
	DataCollection.Reset();
	if (Component.IsValid())
	{
		UOutlinerSettings* OutlinerSettings = GetMutableDefault<UOutlinerSettings>();
		DataCollectionFacade.FillFromGeometryCollectionComponent(*Component, OutlinerSettings->ColumnMode);
	}
}

void FGeometryCollectionTreeItemComponent::RegenerateChildren()
{
	GenerateDataCollection();
	if(Component.IsValid() && Component->GetRestCollection())
	{
		//@todo Fracture:  This is potentially very expensive to refresh with giant trees
		FGeometryCollection* Collection = Component->GetRestCollection()->GetGeometryCollection().Get();

		ItemsByBoneIndex.Empty();
		ChildItems.Empty();

		if (Collection)
		{
			const int32 NumElements = Collection->NumElements(FGeometryCollection::TransformGroup);
			const TManagedArray<int32>& Parents = Collection->Parent;

			RootIndex = FGeometryCollection::Invalid;

			// Add a sub item to the outliner tree for each of the bones/chunks in this GeometryCollection
			for (int32 BoneIndex = 0; BoneIndex < NumElements; BoneIndex++)
			{
				if (FilterBoneIndex(BoneIndex))
				{
					TSharedRef<FGeometryCollectionTreeItemBone> NewItem = MakeShared<FGeometryCollectionTreeItemBone>(BoneIndex, this);
					if (Parents[BoneIndex] == RootIndex)
					{
						// The actual children directly beneath this node are the ones without a parent.  The rest are children of children
						ChildItems.Add(NewItem);
					}

					ItemsByBoneIndex.Add(BoneIndex, NewItem);
				}			
			}

		}
	}
}

void FGeometryCollectionTreeItemComponent::RequestTreeRefresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

void FGeometryCollectionTreeItemComponent::SetHistogramSelection(TArray<int32>& SelectedBones)
{
	HistogramSelection = SelectedBones;
}

bool FGeometryCollectionTreeItemComponent::FilterBoneIndex(int32 BoneIndex) const
{
	if (Component.IsValid())
	{
		const FGeometryCollection* Collection = Component->GetRestCollection()->GetGeometryCollection().Get();
		const TManagedArray<int32>& SimTypes = Collection->SimulationType;
		bool bHasChildren = Collection->Children[BoneIndex].Num() > 0;

		if (SimTypes[BoneIndex] != FGeometryCollection::ESimulationTypes::FST_Clustered)
		{
			// We only display cluster nodes deeper than the view level.
			UFractureSettings* FractureSettings = GetMutableDefault<UFractureSettings>();

			if (FractureSettings->FractureLevel >= 0 && Collection->HasAttribute("Level", FTransformCollection::TransformGroup))
			{
				const TManagedArray<int32>& Level = Collection->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);
				int32 BoneLevel = Level[BoneIndex];
				// bone is not at the right level itself and doesn't have child(ren) at the right level
				if (BoneLevel != FractureSettings->FractureLevel && (!bHasChildren || BoneLevel + 1 != FractureSettings->FractureLevel))
				{
					return false;
				}
			}

			// If anything is selected int the Histogram, we filter by that selection.
			if (HistogramSelection.Num() > 0)
			{
				if (!HistogramSelection.Contains(BoneIndex))
				{
					return false;
				}
			}		
		}
	}
	return true;	
}

const FGeometryCollectionItemDataFacade& FGeometryCollectionTreeItemBone::GetDataCollectionFacade() const
{
	ensure(ParentComponentItem);
	return ParentComponentItem->GetDataCollectionFacade();
}

void FGeometryCollectionTreeItemBone::UpdateItemFromCollection()
{
	const int32 ItemBoneIndex = GetBoneIndex();
	constexpr FLinearColor InvalidColor(0.1f, 0.1f, 0.1f);
	ItemColor = InvalidColor;

	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();

	UOutlinerSettings* OutlinerSettings = GetMutableDefault<UOutlinerSettings>();

	const bool bHasLevelAttribute = DataCollectionFacade.IsLevelAttributeValid();
	if (bHasLevelAttribute && OutlinerSettings->ColorByLevel)
	{
		const TManagedArray<int32>& Level = DataCollectionFacade.GetLevel();
		ItemColor = FSlateColor(FGeometryCollectionTreeItem::GetColorPerDepth((uint32)Level[ItemBoneIndex]));
	}
	else if (DataCollectionFacade.IsSimulationtypeAttributeValid())
	{
		const TManagedArray<int32>& SimulationType = DataCollectionFacade.GetSimulationType();
		switch (SimulationType[ItemBoneIndex])
		{
		case FGeometryCollection::ESimulationTypes::FST_None:
			ItemColor = FLinearColor::Green;
			break;

		case FGeometryCollection::ESimulationTypes::FST_Rigid:
		{
			bool IsVisible = true;
			if (DataCollectionFacade.IsVisibleAttributeValid())
			{
				IsVisible = DataCollectionFacade.GetVisible()[ItemBoneIndex];
			}
			ItemColor = IsVisible ? FSlateColor::UseForeground() : InvalidColor;
			break;
		}

		case FGeometryCollection::ESimulationTypes::FST_Clustered:
			ItemColor = FSlateColor(FColor::Cyan);
			break;

		default:
			ensureMsgf(false, TEXT("Invalid Geometry Collection simulation type encountered."));
			break;
		}
	}
}

TSharedRef<ITableRow> FGeometryCollectionTreeItemBone::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned)
{
	UpdateItemFromCollection();
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	if (bIsPinned)
	{
		return SNew(STableRow<FGeometryCollectionTreeItemPtr>, InOwnerTable)
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.Padding(2.0f, 4.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(BoneIndex))
						.ColorAndOpacity(ItemColor)
					]
				+ SHorizontalBox::Slot()
					.Padding(2.0f, 4.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::FromString(DataCollectionFacade.GetBoneName(BoneIndex)))
					.ColorAndOpacity(ItemColor)
					]
			];
	}
	return SNew(SGeometryCollectionOutlinerRow, InOwnerTable, SharedThis(this));
}

bool FGeometryCollectionTreeItemComponent::IsValid() const
{
	if (bInvalidated)
	{
		return false;
	}
	if (const UGeometryCollectionComponent* Comp = GetComponent())
	{
		if (const UGeometryCollection* RestCollection = Comp->GetRestCollection())
		{
			if (const FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get())
			{
				return Collection->NumElements(FGeometryCollection::TransformGroup) == DataCollectionFacade.GetBoneCount();
			}
		}
	}
	return false;
}

bool FGeometryCollectionTreeItemBone::IsValidBone() const
{
	if (ParentComponentItem && ParentComponentItem->IsValid())
	{
		const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
		return DataCollectionFacade.IsValidBoneIndex(BoneIndex);
	}
	return false;
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeBoneIndexColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(BoneIndex))
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeBoneNameColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(DataCollectionFacade.GetBoneName(BoneIndex)))
				.ColorAndOpacity(ItemColor)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeRelativeSizeColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(2);
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(DataCollectionFacade.GetRelativeSize(BoneIndex), &FormatOptions))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeVolumeColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(2);
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(DataCollectionFacade.GetVolumetricUnit(BoneIndex), &FormatOptions))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeDamagesColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(1)
			.SetMaximumFractionalDigits(1);
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(DataCollectionFacade.GetDamage(BoneIndex), &FormatOptions))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeDamageThresholdColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(DataCollectionFacade.GetDamageThreshold(BoneIndex), &FormatOptions))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeBrokenColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(DataCollectionFacade.IsBroken(BoneIndex) ? TEXT("Broken") : TEXT("")))
				.ColorAndOpacity(ItemColor)
			];
}

static FText FormatRemoveOnBreak_BreakTimer(const FGeometryCollectionItemDataFacade& DataCollectionFacade, int32 BoneIndex)
{
	if (DataCollectionFacade.IsRemoveOnBreakAttributeValid())
	{
		FRemoveOnBreakData Data = DataCollectionFacade.GetRemoveOnBreakData(BoneIndex);
		if (Data.IsEnabled())
		{
			const FVector2f BreakTimer = Data.GetBreakTimer();
			return FText::Format(LOCTEXT("GCOutliner_RemoveOnBreakTimer_Format", "[{0}s, {1}s]"), BreakTimer.X, BreakTimer.Y);
		}
		return FText(LOCTEXT("GCOutliner_RemoveOnBreakTimer_Empty", "[-, -]"));
	}
	return FText();
}

static FText FormatRemoveOnBreak_RemovalTimer(const FGeometryCollectionItemDataFacade& DataCollectionFacade, int32 BoneIndex)
{
	if (DataCollectionFacade.IsRemoveOnBreakAttributeValid())
	{
		FRemoveOnBreakData Data = DataCollectionFacade.GetRemoveOnBreakData(BoneIndex);
		if (Data.IsEnabled())
		{
			const TManagedArray<int32>& SimulationType = DataCollectionFacade.GetSimulationType();
			const bool IsCluster = (SimulationType.Num() > 0) ? (SimulationType[BoneIndex] == FGeometryCollection::ESimulationTypes::FST_Clustered) : false;

			const bool EnableRemovalTimer = !(IsCluster && Data.GetClusterCrumbling());
			const bool ClusterCrumbling = (IsCluster && Data.GetClusterCrumbling());
			if (ClusterCrumbling)
			{
				return FText(LOCTEXT("GCOutliner_ClusterCrumbling_Text", "Cluster Crumbling"));
			}
			const FVector2f RemovalTimer = Data.GetRemovalTimer();
			return FText::Format(LOCTEXT("GCOutliner_RemoveOnBreakTimer_Format", "[{0}s, {1}s]"), RemovalTimer.X, RemovalTimer.Y);
		}
		return FText(LOCTEXT("GCOutliner_RemoveOnBreakTimer_Empty", "[-, -]"));
	}
	return FText();
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakePostBreakTimeColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(FormatRemoveOnBreak_BreakTimer(DataCollectionFacade, BoneIndex))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeRemovalTimeColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(FormatRemoveOnBreak_RemovalTimer(DataCollectionFacade, BoneIndex))
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeImportedCollisionsColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	FText ImportedCollisionText = LOCTEXT("GCOutliner_ImportedCollision_NotAvailable", "-");
	if (DataCollectionFacade.HasSourceCollision(BoneIndex))
	{
		if (DataCollectionFacade.IsSourceCollisionUsed(BoneIndex))
		{
			ImportedCollisionText = LOCTEXT("GCOutliner_ImportedCollision_Used", "Used");
		}
		else
		{
			ImportedCollisionText = LOCTEXT("GCOutliner_ImportedCollision_Available", "Available");
		}
	}
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(ImportedCollisionText)
			.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeConvexCountColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	const int32 ConvexCount = DataCollectionFacade.GetConvexCount(BoneIndex);
	const bool bIsUnionOfConvex = (ConvexCount < 0);

	const FText ItemText = bIsUnionOfConvex? FText::Format(LOCTEXT("GCOutliner_ConvexCount_Format", "Union of {0}"), FMath::Abs(ConvexCount)): FText::AsNumber(ConvexCount);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(ItemText)
		.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeTriangleCountColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	const int32 TriangleCount = DataCollectionFacade.GetTriangleCount(BoneIndex);
	const bool bIsSumOfChildren = (TriangleCount < 0);

	const FText ItemText = bIsSumOfChildren ? FText::Format(LOCTEXT("GCOutliner_TriangleCount_Format", "({0})"), FMath::Abs(TriangleCount)) : FText::AsNumber(TriangleCount);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(ItemText)
		.ColorAndOpacity(ItemColor)
		];
}
TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeVertexCountColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	const int32 VertexCount = DataCollectionFacade.GetVertexCount(BoneIndex);
	const bool bIsSumOfChildren = (VertexCount < 0);

	const FText ItemText = bIsSumOfChildren ? FText::Format(LOCTEXT("GCOutliner_VertexCount_Format", "({0})"), FMath::Abs(VertexCount)) : FText::AsNumber(VertexCount);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(12.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(ItemText)
		.ColorAndOpacity(ItemColor)
		];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeInitialStateColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			[
				SNew(STextBlock)
				.Text(GetTextFromInitialDynamicState(DataCollectionFacade.GetInitialState(BoneIndex)))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeAnchoredColumnWidget() const
{
	const FGeometryCollectionItemDataFacade& DataCollectionFacade = GetDataCollectionFacade();
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
	[
				SNew(STextBlock)
				.Text(GetTextFromAnchored(DataCollectionFacade.IsAnchored(BoneIndex)))
				.ColorAndOpacity(ItemColor)
			];
}

TSharedRef<SWidget> FGeometryCollectionTreeItemBone::MakeEmptyColumnWidget() const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(12.f, 0.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(ItemColor)
			];
}

void FGeometryCollectionTreeItemBone::GetChildren(FGeometryCollectionTreeItemList& OutChildren)
{
	ParentComponentItem->GetChildrenForBone(*this, OutChildren);
}

bool FGeometryCollectionTreeItemBone::HasChildren() const 
{
	return ParentComponentItem->HasChildrenForBone(*this);
}


#undef LOCTEXT_NAMESPACE // "ChaosEditor"
