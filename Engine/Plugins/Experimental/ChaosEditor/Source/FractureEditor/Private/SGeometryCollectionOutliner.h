// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STreeView.h"
#include "Input/DragAndDrop.h"

#include "Misc/Attribute.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"

#include "FractureEditorModeToolkit.h"

#include "SGeometryCollectionOutliner.generated.h"

class FGeometryCollection;
class FGeometryCollectionTreeItem;
class FGeometryCollectionTreeItemBone;
class SGeometryCollectionOutliner;
class UToolMenu;

typedef TArray<TSharedPtr<FGeometryCollectionTreeItem>> FGeometryCollectionTreeItemList;
typedef TSharedPtr<FGeometryCollectionTreeItem> FGeometryCollectionTreeItemPtr;


UENUM(BlueprintType)
enum class EOutlinerItemNameEnum : uint8
{
	BoneName = 0					UMETA(DisplayName = "Bone Name"),
	BoneIndex = 1					UMETA(DisplayName = "Bone Index"),
};

/** Settings for Outliner configuration. **/
UCLASS()
class UOutlinerSettings : public UObject
{

	GENERATED_BODY()
public:
	UOutlinerSettings(const FObjectInitializer& ObjInit);

	/** whether to use level coloring */
	UPROPERTY(EditAnywhere, Category = OutlinerSettings, meta = (DisplayName = "Color By Level"))
	bool ColorByLevel;

	/** the column to be display in the outliner */
	UPROPERTY(EditAnywhere, Category = OutlinerSettings, meta = (DisplayName = "Column Mode"))
	EOutlinerColumnMode ColumnMode;
};


class FGeometryCollectionTreeItem : public TSharedFromThis<FGeometryCollectionTreeItem>
{
public:
	virtual ~FGeometryCollectionTreeItem() = default;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false) = 0;
	virtual void GetChildren(FGeometryCollectionTreeItemList& OutChildren) = 0;
	virtual UGeometryCollectionComponent* GetComponent() const = 0;

	virtual int32 GetBoneIndex() const { return INDEX_NONE; }

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent) { return FReply::Unhandled(); }
	
	virtual void OnDragEnter(FDragDropEvent const& InDragDropEvent) {};
	virtual void OnDragLeave(FDragDropEvent const& InDragDropEvent);

	//void MakeDynamicStateMenu(UToolMenu* Menu, SGeometryCollectionOutliner& Outliner);
	void GenerateContextMenu(UToolMenu* Menu, SGeometryCollectionOutliner& Outliner);

	static FColor GetColorPerDepth(uint32 Depth);
};

using FRemoveOnBreakData = GeometryCollection::Facades::FRemoveOnBreakData;

class FGeometryCollectionItemDataFacade
{
public:
	FGeometryCollectionItemDataFacade(FManagedArrayCollection& InCollection);

	void FillFromGeometryCollectionComponent(const UGeometryCollectionComponent& GeometryCollectionComponent, EOutlinerColumnMode ColumnMode);

	bool IsLevelAttributeValid() const { return LevelAttribute.IsValid(); }
	bool IsSimulationtypeAttributeValid() const { return SimulationTypeAttribute.IsValid(); }
	bool IsVisibleAttributeValid() const { return VisibleAttribute.IsValid(); }
	bool IsRemoveOnBreakAttributeValid() const { return RemoveOnBreakAttribute.IsValid(); }

	const TManagedArray<int32>& GetLevel() const { return LevelAttribute.Get(); }
	const TManagedArray<int32>& GetSimulationType() const { return SimulationTypeAttribute.Get(); }
	const TManagedArray<bool>& GetVisible() const { return VisibleAttribute.Get(); }

	bool IsValidBoneIndex(int32 BoneIndex) const;
	FString GetBoneName(int32 Index) const;
	int32 GetBoneCount() const;
	float GetRelativeSize(int32 Index) const;
	float GetVolumetricUnit(int32 Index) const;
	int32 GetInitialState(int32 Index) const;
	bool IsAnchored(int32 Index) const;
	float GetDamage(int32 Index) const;
	float GetDamageThreshold(int32 Index) const;
	bool IsBroken(int32 Index) const;
	FRemoveOnBreakData GetRemoveOnBreakData(int32 Index) const;
	bool HasSourceCollision(int32 Index) const;
	bool IsSourceCollisionUsed(int32 Index) const;
	int32 GetConvexCount(int32 Index) const;
	int32 GetTriangleCount(int32 Index) const;
	int32 GetVertexCount(int32 Index) const;

private:
	FManagedArrayCollection&			DataCollection;
	TManagedArrayAccessor<FString>		BoneNameAttribute;
	TManagedArrayAccessor<int32>		LevelAttribute;
	//TManagedArrayAccessor<TArray<int32>>ChildrenAttribute;
	TManagedArrayAccessor<bool>			VisibleAttribute;
	TManagedArrayAccessor<int32>		InitialStateAttribute;
	TManagedArrayAccessor<float>		RelativeSizeAttribute;
	TManagedArrayAccessor<float>		VolumetricUnitAttribute;
	TManagedArrayAccessor<bool>			AnchoredAttribute;
	TManagedArrayAccessor<float>		DamageAttribute;
	TManagedArrayAccessor<float>		DamageThresholdAttribute;
	TManagedArrayAccessor<bool>			BrokenStateAttribute;
	TManagedArrayAccessor<FVector4f>	RemoveOnBreakAttribute;
	TManagedArrayAccessor<int32>		SimulationTypeAttribute;
	TManagedArrayAccessor<bool>			HasSourceCollisionAttribute;
	TManagedArrayAccessor<bool>			SourceCollisionUsedAttribute;
	TManagedArrayAccessor<int32>		ConvexCountAttribute;
	TManagedArrayAccessor<int32>		TriangleCountAttribute;
	TManagedArrayAccessor<int32>		VertexCountAttribute;
};

class FGeometryCollectionTreeItemComponent : public FGeometryCollectionTreeItem
{
public:
	FGeometryCollectionTreeItemComponent(UGeometryCollectionComponent* InComponent, TSharedPtr<STreeView<FGeometryCollectionTreeItemPtr>> InTreeView)
		: Component(InComponent)
		, TreeView(InTreeView)
		, DataCollectionFacade(DataCollection)
	{
		RegenerateChildren();
	}
	
	virtual ~FGeometryCollectionTreeItemComponent() = default;

	/** FGeometryCollectionTreeItem interface */
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	virtual void GetChildren(FGeometryCollectionTreeItemList& OutChildren) override;
	virtual UGeometryCollectionComponent* GetComponent() const override { return Component.Get(); }


	FGeometryCollectionTreeItemPtr GetItemFromBoneIndex(int32 BoneIndex) const;
	 
	void GetChildrenForBone(FGeometryCollectionTreeItemBone& BoneItem, FGeometryCollectionTreeItemList& OutChildren) const;
	bool HasChildrenForBone(const FGeometryCollectionTreeItemBone& BoneItem) const;

	void ExpandAll();
	void GenerateDataCollection();
	void RegenerateChildren();
	void RequestTreeRefresh();

	void SetHistogramSelection(TArray<int32>& SelectedBones);

	const FGeometryCollectionItemDataFacade& GetDataCollectionFacade() const { return DataCollectionFacade; }
	FGeometryCollectionItemDataFacade& GetDataCollectionFacade() { return DataCollectionFacade; }

	bool IsValid() const;

	// Mark item as unused/invalid; helpful because slate defers destroying tree items and can still run callbacks on them until tick
	void Invalidate()
	{
		bInvalidated = true;
	}

private:
	bool FilterBoneIndex(int32 BoneIndex) const;

private:
	TWeakObjectPtr<UGeometryCollectionComponent> Component;

	TSharedPtr<STreeView<FGeometryCollectionTreeItemPtr>> TreeView;

	/** The direct children under this component */
	TArray<FGeometryCollectionTreeItemPtr> ChildItems;
	TMap<int32, FGeometryCollectionTreeItemPtr> ItemsByBoneIndex;
	int32 RootIndex;

	TArray<int32> HistogramSelection;

	// collection used to store the displayed information
	FManagedArrayCollection DataCollection;
	FGeometryCollectionItemDataFacade DataCollectionFacade;

	// track whether the item has been explicitly invalidated
	bool bInvalidated = false;

};

class FGeometryCollectionTreeItemBone : public FGeometryCollectionTreeItem
{

public:
	FGeometryCollectionTreeItemBone(const int32 InBoneIndex, FGeometryCollectionTreeItemComponent* InParentComponentItem)
		: BoneIndex(InBoneIndex)
		, ParentComponentItem(InParentComponentItem)
		, ItemColor(FSlateColor::UseForeground())
	{}

	/** FGeometryCollectionTreeItem interface */
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, bool bIsPinned = false);
	TSharedRef<SWidget> MakeBoneIndexColumnWidget() const;
	TSharedRef<SWidget> MakeBoneNameColumnWidget() const;
	TSharedRef<SWidget> MakeRelativeSizeColumnWidget() const;
	TSharedRef<SWidget> MakeVolumeColumnWidget() const;
	TSharedRef<SWidget> MakeDamagesColumnWidget() const;
	TSharedRef<SWidget> MakeDamageThresholdColumnWidget() const;
	TSharedRef<SWidget> MakeBrokenColumnWidget() const;
	TSharedRef<SWidget> MakeInitialStateColumnWidget() const;
	TSharedRef<SWidget> MakeAnchoredColumnWidget() const;
	TSharedRef<SWidget> MakePostBreakTimeColumnWidget() const;
	TSharedRef<SWidget> MakeRemovalTimeColumnWidget() const;
	TSharedRef<SWidget> MakeImportedCollisionsColumnWidget() const;
	TSharedRef<SWidget> MakeConvexCountColumnWidget() const;
	TSharedRef<SWidget> MakeTriangleCountColumnWidget() const;
	TSharedRef<SWidget> MakeVertexCountColumnWidget() const;
	TSharedRef<SWidget> MakeEmptyColumnWidget() const;
	virtual void GetChildren(FGeometryCollectionTreeItemList& OutChildren) override;
	bool IsValidBone() const;
	virtual int32 GetBoneIndex() const override { return BoneIndex; }
	virtual UGeometryCollectionComponent* GetComponent() const { return ParentComponentItem->GetComponent(); }
	bool HasChildren() const;
	
protected:
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragEnter(FDragDropEvent const& InDragDropEvent) override;

private:
	void UpdateItemFromCollection();

	const FGeometryCollectionItemDataFacade& GetDataCollectionFacade() const;
	
private:
	const int32 BoneIndex;
	FGeometryCollectionTreeItemComponent* ParentComponentItem;
	FSlateColor ItemColor;
};

typedef TSharedPtr<class FGeometryCollectionTreeItemBone> FGeometryCollectionTreeItemBonePtr;

namespace SGeometryCollectionOutlinerColumnID
{
	const FName BoneIndex("Index");
	const FName BoneName("Name");
	// State and Size column mode
	const FName RelativeSize("Relative Size");
	const FName Volume("Volume");
	const FName InitialState("Initial State");
	const FName Anchored("Anchored");
	// Damage Column Mode
	const FName Damage("Damage");
	const FName DamageThreshold("DamageThreshold");
	const FName Broken("Broken");
	// Removal Column Mode
	const FName PostBreakTime("PostBreakTime");
	const FName RemovalTime("RemovalTime");
	// Collision Column Mode
	const FName ConvexCount("Convex Count");
	const FName ImportedCollisions("ImportedCollisions");
	// Geometry Column Mode
	const FName VertexCount("Vertex Count");
	const FName TriangleCount("Triangle Count");
}

class SGeometryCollectionOutlinerRow : public SMultiColumnTableRow<FGeometryCollectionTreeItemBonePtr>
{
protected:
	FGeometryCollectionTreeItemBonePtr Item;

public:
	SLATE_BEGIN_ARGS(SGeometryCollectionOutlinerRow) {}
	SLATE_END_ARGS()

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, FGeometryCollectionTreeItemBonePtr InItemToEdit)
	{
		Item = InItemToEdit;
		SMultiColumnTableRow<FGeometryCollectionTreeItemBonePtr>::Construct(
		FSuperRowType::FArguments()
			.OnDragDetected(Item.Get(), &FGeometryCollectionTreeItem::OnDragDetected)
			.OnDrop(Item.Get(), &FGeometryCollectionTreeItem::OnDrop)
			.OnDragEnter(Item.Get(), &FGeometryCollectionTreeItem::OnDragEnter)
			.OnDragLeave(Item.Get(), &FGeometryCollectionTreeItem::OnDragLeave)
		, OwnerTableView);
	}
};

class SGeometryCollectionOutliner: public SCompoundWidget
{
	DECLARE_DELEGATE_TwoParams(FOnBoneSelectionChanged, UGeometryCollectionComponent*, TArray<int32>&);

	SLATE_BEGIN_ARGS( SGeometryCollectionOutliner ) 
	{}

		SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged)

	SLATE_END_ARGS() 

public:
	void Construct(const FArguments& InArgs);

	void RegenerateItems();
	void RegenerateHeader();
	void RegenerateRootData();

	TSharedRef<ITableRow> MakeTreeRowWidget(FGeometryCollectionTreeItemPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGeneratePinnedRowWidget(FGeometryCollectionTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable, bool bPinned);
	void OnGetChildren(TSharedPtr<FGeometryCollectionTreeItem> InInfo, TArray< TSharedPtr<FGeometryCollectionTreeItem> >& OutChildren);
	TSharedPtr<SWidget> OnOpenContextMenu();

	void UpdateGeometryCollection();
	void SetComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents);
	void SetBoneSelection(UGeometryCollectionComponent* RootComponent, const TArray<int32>& InSelection, bool bClearCurrentSelection, int32 FocusBoneIdx = -1);
	int32 GetBoneSelectionCount() const;
	int32 GetBoneDisplayedCount() const;

	void SetInitialDynamicState(int32 InDynamicState);
	void SetAnchored(bool bAnchored);

	void ExpandAll();
	void ExpandRecursive(TSharedPtr<FGeometryCollectionTreeItem> TreeItem, bool bInExpansionState) const;

	// Set the histogram filter on the component matching RootComponent.
	void SetHistogramSelection(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones);

private:
	void OnSelectionChanged(FGeometryCollectionTreeItemPtr Item, ESelectInfo::Type SelectInfo);
	
private:
	TSharedPtr<STreeView<FGeometryCollectionTreeItemPtr>> TreeView;
	TSharedPtr< SHeaderRow > HeaderRowWidget;
	TArray<TSharedPtr<FGeometryCollectionTreeItemComponent>> RootNodes;
	FOnBoneSelectionChanged BoneSelectionChangedDelegate;
	bool bPerformingSelection;
};
