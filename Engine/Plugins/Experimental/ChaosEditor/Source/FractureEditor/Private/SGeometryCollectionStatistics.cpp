// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCollectionStatistics.h"
#include "SGeometryCollectionOutliner.h"

#include "Widgets/Layout/SBox.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SGeometryCollectionStatistics"

namespace GeometryCollectionStatisticsUI
{
	static const FName NameLabel(TEXT("Stat"));
	static const FName NumBonesLabel(TEXT("NumBones"));
	static const FName NumConvexLabel(TEXT("NumConvex"));
}

class SGeometryCollectionStatisticsRow : public SMultiColumnTableRow<TSharedPtr<int32>>
{
	SLATE_BEGIN_ARGS(SGeometryCollectionStatisticsRow) {}
		SLATE_ARGUMENT(FText, Name)
		SLATE_ARGUMENT(FText, NumBones)
		SLATE_ARGUMENT(FText, NumConvex)
		SLATE_ARGUMENT(FSlateColor, TextColor)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Name = InArgs._Name;
		NumBones = InArgs._NumBones;
		NumConvex = InArgs._NumConvex;
		TextColor = InArgs._TextColor;
		SMultiColumnTableRow<TSharedPtr<int32> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		FText ColumnText;

		if (ColumnName == GeometryCollectionStatisticsUI::NameLabel)
		{
			ColumnText = Name;
		}
		else if (ColumnName == GeometryCollectionStatisticsUI::NumBonesLabel)
		{
			ColumnText = NumBones;
		}
		else if (ColumnName == GeometryCollectionStatisticsUI::NumConvexLabel)
		{
			ColumnText = NumConvex;
		}

		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			[
				SNew(STextBlock)
				.ColorAndOpacity(TextColor)
				.Text(ColumnText)
			];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	FText Name;
	FText NumBones;
	FText NumConvex;
	FSlateColor TextColor;
};

void SGeometryCollectionStatistics::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SAssignNew(StatListView, SListView<FListItemPtr>)
			.ListItemsSource(&StatItems)
			.OnGenerateRow(this, &SGeometryCollectionStatistics::HandleGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(GeometryCollectionStatisticsUI::NameLabel)
				.DefaultLabel(LOCTEXT("NameColumnHeaderName", "Name"))
				+SHeaderRow::Column(GeometryCollectionStatisticsUI::NumBonesLabel)
				.DefaultLabel(LOCTEXT("NumBonesColumnHeaderName", "Num Bones"))
				+ SHeaderRow::Column(GeometryCollectionStatisticsUI::NumConvexLabel)
				.DefaultLabel(LOCTEXT("NumConvexColumnHeaderName", "Num Convex"))

			)
		];
}

void SGeometryCollectionStatistics::Refresh()
{
	StatListView->RequestListRefresh();
}

void SGeometryCollectionStatistics::SetStatistics(const FGeometryCollectionStatistics& Stats)
{
	UOutlinerSettings* OutlinerSettings = GetMutableDefault<UOutlinerSettings>();
	StatItems.Reset();
	
	for (int32 Level = 0; Level < Stats.CountsPerLevel.Num(); ++Level)
	{
		FSlateColor TextColor = FSlateColor::UseForeground();
		if (OutlinerSettings->ColorByLevel)
		{
			TextColor = FGeometryCollectionTreeItem::GetColorPerDepth(Level);
		}
		FStatRow Row
		{
			FText::Format(LOCTEXT("Level", "Level {0}"), FText::AsNumber(Level)),
			FText::AsNumber(Stats.CountsPerLevel[Level]),
			FText::AsNumber(Stats.ConvexCountPerLevel.IsValidIndex(Level)? Stats.ConvexCountPerLevel[Level]: 0),
			TextColor
		};
		StatItems.Emplace(MakeShared<FStatRow>(MoveTemp(Row)));
	}

	FStatRow Row
	{
		FText(LOCTEXT("Embedded", "Embedded")),
		FText::AsNumber(Stats.EmbeddedCount),
		FText::AsNumber(0),
		FSlateColor::UseForeground()
	};
	StatItems.Emplace(MakeShared<FStatRow>(MoveTemp(Row)));


	StatListView->RequestListRefresh();
}

TSharedRef<ITableRow> SGeometryCollectionStatistics::HandleGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{	
	return SNew(SGeometryCollectionStatisticsRow, OwnerTable)
		.Name(InItem->Name)
		.NumBones(InItem->NumBones)
		.NumConvex(InItem->NumConvex)
		.TextColor(InItem->TextColor);
}

#undef LOCTEXT_NAMESPACE /** SGeometryCollectionStatistics */
