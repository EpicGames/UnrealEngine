// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

struct FGeometryCollectionStatistics
{
	TArray<uint32> CountsPerLevel;
	TArray<uint32> ConvexCountPerLevel;
	uint32 EmbeddedCount = 0;
};

class SGeometryCollectionStatistics : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGeometryCollectionStatistics)
	{}
	SLATE_END_ARGS()

	/**
	* Constructs a geom collection statistics widget.
	*
	* @param InArgs The Slate argument list.
	*/
	void Construct(const FArguments& InArgs);

	void SetStatistics(const FGeometryCollectionStatistics& Stats);

	void Refresh();

private:

	/** Holds basic information about a stat. */
	struct FStatRow
	{
		FText Name;
		FText NumBones;
		FText NumConvex;
		FSlateColor TextColor;
	};

	using FListItemPtr = TSharedPtr<FStatRow>;

	/** Callback for generating package info rows. */
	TSharedRef<ITableRow> HandleGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

private:

	/** Holds the ListView of package details rows. */
	TSharedPtr<SListView<FListItemPtr>>	StatListView;

	/** Holds the package details rows. */
	TArray<FListItemPtr> StatItems;
};