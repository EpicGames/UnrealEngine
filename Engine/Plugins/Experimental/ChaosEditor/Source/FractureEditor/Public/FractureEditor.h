// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "Toolkits/AssetEditorToolkit.h"

class FExtensibilityManager;

class FFractureEditorModule : public IModuleInterface
{

  public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Extensibility Manager for Adding Fracture Tools **/
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() const;

  private:
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};
