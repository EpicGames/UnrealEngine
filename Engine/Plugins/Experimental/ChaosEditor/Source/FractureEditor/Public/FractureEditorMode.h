// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "EditorUndoClient.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "Engine/DeveloperSettings.h"

#include "FractureEditorMode.generated.h"

class UGeometryCollectionComponent;

namespace FractureTransactionContexts
{
	static const TCHAR SelectBoneContext[] = TEXT("SelectGeometryCollectionBone");
};

UCLASS(Transient)
class UFractureEditorMode : public UBaseLegacyWidgetEdMode, public FEditorUndoClient, public ILegacyEdModeSelectInterface
{
	GENERATED_BODY()
public:
	const static FEditorModeID EM_FractureEditorModeId;

	UFractureEditorMode();
	virtual ~UFractureEditorMode();

	using UEdMode::PostUndo;
	using FEditorUndoClient::PostUndo;

	// UEdMode interface
	virtual void Enter() override;
	virtual void Exit() override;

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;

	virtual void CreateToolkit() override;
	bool UsesToolkits() const override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;

	// Helper for HandleClick, exposed to allow other code paths to handle clicks via the same code path
	bool SelectFromClick(HHitProxy* HitProxy, bool bCtrlDown, bool bShiftDown);

	virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const;
	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	// End of UEdMode interface

	// ILegacyEdModeViewportInterface
	virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	// ILegacyEdModeSelectInterface
	virtual bool BoxSelect(FBox& InBox, bool InSelect = true) override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect) override;

	// Helpers for FrustumSelect + to expose similar selection functionality to other code
	bool UpdateSelectionInFrustum(const FConvexVolume& InFrustum, AActor* Actor, bool bStrictDragSelection, bool bAppend, bool bRemove);
	bool UpdateSelection(const TArray<int32>& PreviousSelection, TArray<int32>& Bones, bool bAppend, bool bRemove);

	// FEditorUndoClient interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	// Used to update outliner when the selection hasn't change -- needed because the outliner is not built yet when Enter() is called
	// TODO: We may be able to remove this function if we can find a way to invoke Enter's current selection-update logic after the outliner UI is built
	void RefreshOutlinerWithCurrentSelection();

private:
	void OnUndoRedo();
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);
	void GetComponentGlobalBounds(UGeometryCollectionComponent* GeometryCollectionComponent, TArray<FBox>& BoundsPerBone) const;
	void SelectionStateChanged();
	
	/** Handle package reloading (might be our geometry collection) */
	void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	static FConvexVolume TransformFrustum(const FConvexVolume& InFrustum, const FMatrix& InMatrix);
	static FConvexVolume GetVolumeFromBox(const FBox &InBox);

	void ConfigureRealTimeViewportsOverride(bool bEnable);

	void RenderSelectionAsBoxes(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);

private:
	/** This selection set is updated from actor selection changed event.  We change state on components as they are selected so we have to maintain or own list **/
	TArray<TWeakObjectPtr<UGeometryCollectionComponent>> SelectedGeometryComponents;

	// Return an array of all selected geometry collection components that are still valid/non-null
	TArray<UGeometryCollectionComponent*> GetValidSelectedGeometryComponents();

	// Hack: We have to set this to work around the editor defaulting to orbit around selection and breaking our custom per-bone orbiting
	mutable TOptional<FVector> CustomOrbitPivot;
};




/**
 * Defines a color to be used for a particular Tool Palette Section
 */
USTRUCT()
struct FFractureModeCustomSectionColor
{
	GENERATED_BODY()

	/** Name of Section in Fracture Mode Tool Palette */
	UPROPERTY(EditAnywhere, Category = "SectionColor")
	FString SectionName = TEXT("");

	/** Custom Header Color */
	UPROPERTY(EditAnywhere, Category = "SectionColor")
	FLinearColor Color = FLinearColor::Gray;
};


/**
 * Defines a color to be used for a particular Tool Palette Section
 */
USTRUCT()
struct FFractureModeCustomToolColor
{
	GENERATED_BODY()

	/**
	 * Name of Section or Tool in Fracture Mode Tool Palette
	 *
	 * Format:
	 * SectionName        (Specifies a default color for all tools in the section.)
	 * SectionName.ToolName        (Specifies an override color for a specific tool in the given section.)
	 */
	UPROPERTY(EditAnywhere, Category = "ToolColor")
	FString ToolName = TEXT("");

	/** Custom Tool Color */
	UPROPERTY(EditAnywhere, Category = "ToolColor")
	FLinearColor Color = FLinearColor::Gray;
};


UCLASS(MinimalAPI, config = Editor)
class UFractureModeCustomizationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// UDeveloperSettings overrides

	virtual FName GetContainerName() const { return FName("Editor"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("FractureEditor"); }

	virtual FText GetSectionText() const override
	{
		return NSLOCTEXT("FractureEditorMode", "FractureModeSettingsName", "Fracture Mode");
	}

	virtual FText GetSectionDescription() const override
	{
		return NSLOCTEXT("FractureEditorMode", "FractureModeSettingsDescription", "Configure the Fracture Editor Mode plugin");
	}

public:


	/** Add the names of Fracture Mode Tool Palette Sections to have them appear at the top of the Tool Palette, in the order listed below. */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|UI Customization")
	TArray<FString> ToolSectionOrder;

	/** Tool Names listed in the array below will appear in a Favorites section at the top of the Fracture Mode Tool Palette */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|UI Customization")
	TArray<FString> ToolFavorites;

	/** Custom Section Header Colors for listed Sections in the Fracture Mode Tool Palette */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|UI Customization")
	TArray<FFractureModeCustomSectionColor> SectionColors;

	/**
	 * Custom Tool Colors for listed Tools in the Fracture Mode Tool Palette.
	 * 
	 * Format:
	 * SectionName        (Specifies a default color for all tools in the section.)
	 * SectionName.ToolName        (Specifies an override color for a specific tool in the given section.)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|UI Customization")
	TArray<FFractureModeCustomToolColor> ToolColors;
};

