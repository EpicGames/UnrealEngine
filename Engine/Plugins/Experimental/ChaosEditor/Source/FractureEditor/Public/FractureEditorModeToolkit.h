// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/Real.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Toolkits/BaseToolkit.h"
#include "UnrealEdMisc.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "IDetailCustomization.h"

#include "FractureEditorModeToolkit.generated.h"

#define UE_API FRACTUREEDITOR_API

class IDetailsView;
class IPropertyHandle;
class IStructureDetailsView;
class SScrollBox;
class FFractureToolContext;
class FGeometryCollection;
struct FPropertyChangedEvent;
class SGeometryCollectionOutliner;
class SGeometryCollectionHistogram;
class SGeometryCollectionStatistics;
struct FGeometryCollectionStatistics;
class AGeometryCollectionActor;
class UGeometryCollectionComponent;
class UGeometryCollection;
class FFractureEditorModeToolkit;
class UFractureActionTool;
class UFractureModalTool;
enum class EMapChangeType : uint8;

namespace GeometryCollection
{
enum class ESelectionMode: uint8;
}

namespace Chaos
{
	template<class T, int d>
	class TParticles;
}

class FFractureViewSettingsCustomization : public IDetailCustomization
{
public:
	FFractureViewSettingsCustomization(FFractureEditorModeToolkit* FractureToolkit);
	static TSharedRef<IDetailCustomization> MakeInstance(FFractureEditorModeToolkit* FractureToolkit);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FFractureEditorModeToolkit* Toolkit;
};

class FHistogramSettingsCustomization : public IDetailCustomization
{
public:
	FHistogramSettingsCustomization(FFractureEditorModeToolkit* FractureToolkit);
	static TSharedRef<IDetailCustomization> MakeInstance(FFractureEditorModeToolkit* FractureToolkit);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FFractureEditorModeToolkit* Toolkit;
};

class FOutlinerSettingsCustomization : public IDetailCustomization
{
public:
	FOutlinerSettingsCustomization(FFractureEditorModeToolkit* FractureToolkit);
	static TSharedRef<IDetailCustomization> MakeInstance(FFractureEditorModeToolkit* FractureToolkit);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FFractureEditorModeToolkit* Toolkit;
};

struct FTextAndSlateColor
{
	FTextAndSlateColor(const FText& InText, const FSlateColor& InColor)
		: Text(InText)
		, Color(InColor)
	{}
	FText Text;
	FSlateColor Color;
};

UENUM(BlueprintType)
enum class EOutlinerColumnMode : uint8
{
	State = 0				UMETA(DisplayName = "State"),
	Damage = 1				UMETA(DisplayName = "Damage"),
	Removal = 2				UMETA(DisplayName = "Removal"),
	Collision = 3			UMETA(DisplayName = "Collision"),
	Size = 4				UMETA(DisplayName = "Size"),
	Geometry = 5			UMETA(DisplayName = "Geometry")
};

class FFractureEditorModeToolkit : public FModeToolkit, public FGCObject
{
public:

	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	UE_API FFractureEditorModeToolkit();
	UE_API ~FFractureEditorModeToolkit();
	
	/** FModeToolkit interface */
	UE_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	/** IToolkit interface */
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual class FEdMode* GetEditorMode() const override;
	UE_API virtual TSharedPtr<class SWidget> GetInlineContent() const override;

	/** FGCObject interface */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFractureEditorModeToolkit");
	}

	UE_API void ExecuteAction(UFractureActionTool* InActionTool);
	UE_API bool CanExecuteAction(UFractureActionTool* InActionTool) const;

	UE_API bool CanSetModalTool(UFractureModalTool* InActiveTool) const;
	UE_API void SetActiveTool(UFractureModalTool* InActiveTool);
	UE_API UFractureModalTool* GetActiveTool() const;
	UE_API bool IsActiveTool(UFractureModalTool* InActiveTool);
	UE_API void ShutdownActiveTool();

	UE_API void Shutdown();

	UE_API void SetOutlinerComponents(const TArray<UGeometryCollectionComponent*>& InNewComponents);
	UE_API void SetBoneSelection(UGeometryCollectionComponent* InRootComponent, const TArray<int32>& InSelectedBones, bool bClearCurrentSelection, int32 FocusBoneIdx = -1);

	// View Settings
	UE_API float GetExplodedViewValue() const;
	UE_API int32 GetLevelViewValue() const;
	UE_API bool GetHideUnselectedValue() const;
	UE_API void OnSetExplodedViewValue(float NewValue);
	UE_API void OnSetLevelViewValue(int32 NewValue);

	UE_API void OnExplodedViewValueChanged();
	UE_API void OnLevelViewValueChanged();
	UE_API void OnHideUnselectedChanged();
	UE_API void UpdateHideForComponent(UGeometryCollectionComponent* Component);

	// Update any View Property Changes 
	UE_API void OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent );

	UE_API TSharedRef<SWidget> GetLevelViewMenuContent(TSharedRef<IPropertyHandle> PropertyHandle);
	UE_API TSharedRef<SWidget> GetViewMenuContent();

	UE_API void ToggleShowBoneColors();

	UE_API void ViewUpOneLevel();
	UE_API void ViewDownOneLevel();

	// Modal Command Callback
	UE_API FReply OnModalClicked();
	UE_API bool CanExecuteModal() const;

	static UE_API void GetSelectedGeometryCollectionComponents(TSet<UGeometryCollectionComponent*>& GeomCompSelection);

	static UE_API void AddAdditionalAttributesIfRequired(UGeometryCollection* GeometryCollectionObject);
	UE_API int32 GetLevelCount();

	UE_API void GetStatisticsSummary(FGeometryCollectionStatistics& Stats) const;
	UE_API FText GetSelectionInfo() const;

	/** Returns the number of Mode specific tabs in the mode toolbar **/ 
	UE_API const static TArray<FName> PaletteNames;
	UE_API virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const;
	UE_API virtual FText GetToolPaletteDisplayName(FName PaletteName) const; 

	/* Exclusive Tool Palettes only allow users to use tools from one palette at a time */
	virtual bool HasExclusiveToolPalettes() const { return false; }

	/* Integrated Tool Palettes show up in the same panel as their details */
	virtual bool HasIntegratedToolPalettes() const { return false; }

	UE_API void SetInitialPalette();
	UE_API virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder);
	UE_API virtual void OnToolPaletteChanged(FName PaletteName) override;

	/** Modes Panel Header Information **/
	UE_API virtual FText GetActiveToolDisplayName() const;
	UE_API virtual FText GetActiveToolMessage() const;

	UE_API void UpdateExplodedVectors(UGeometryCollectionComponent* GeometryCollectionComponent) const;

	UE_API void RefreshOutliner();
	
	UE_API void RegenerateOutliner();
	UE_API void RegenerateHistogram();

	TSharedPtr<SWidget> ExplodedViewWidget;
	TSharedPtr<SWidget> LevelViewWidget;
	TSharedPtr<SWidget> ShowBoneColorsWidget;

	UE_API void SetOutlinerColumnMode(EOutlinerColumnMode ColumnMode);

	// function to poll whether the geometry collection data cached by the outliner is stale compared to the current geometry (using quick heuristics such as bone counts)
	UE_API bool IsCachedOutlinerGeometryStale(const TArray<TWeakObjectPtr<UGeometryCollectionComponent>>& SelectedComponents) const;

protected:
	/** FModeToolkit interface */
	UE_API virtual void RequestModeUITabs() override;
	UE_API virtual void InvokeUI() override;

	UE_API TSharedRef<SDockTab> CreateHierarchyTab(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> CreateStatisticsTab(const FSpawnTabArgs& Args);
	static UE_API bool IsGeometryCollectionSelected();
	static UE_API bool IsSelectedActorsInEditorWorld();	

	UE_API void InvalidateCachedDetailPanelState(UObject* ChangedObject);

	// Invalidate the hit proxies for all level viewports; we need to do this after updating geometry collection(s)
	UE_API void InvalidateHitProxies();

private:
	static UE_API void UpdateGeometryComponentAttributes(UGeometryCollectionComponent* Component);

	UE_API void OnOutlinerBoneSelectionChanged(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones);
	UE_API void OnHistogramBoneSelectionChanged(UGeometryCollectionComponent* RootComponent, TArray<int32>& SelectedBones);
	UE_API void BindCommands();

	UE_API void SetHideForUnselected(UGeometryCollectionComponent* GCComp);

	/** Callback for map changes. */
	UE_API void HandleMapChanged(UWorld* NewWorld, EMapChangeType MapChangeType);

	UE_API FReply OnRefreshOutlinerButtonClicked();

	UE_API TSharedRef<SWidget> MakeMenu_FractureModeConfigSettings();
	UE_API void UpdateAssetLocationMode(TSharedPtr<FString> NewString);
	UE_API void UpdateAssetPanelFromSettings();
	UE_API void OnProjectSettingsModified();

	UE_API void UpdateOutlinerHeader();

	UE_API void CreateVariableOverrideDetailView();
	UE_API void OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void RefreshVariableOverrideDetailView(const UGeometryCollection* RestCollection);

	UE_API FReply OnDataflowOverridesUpdateAsset();
	UE_API bool CanDataflowOverridesUpdateAsset() const;
	UE_API bool GetDataflowOverridesUpdateAssetEnabled() const;

	void CreateDataflowProgressBar();

private:
	TObjectPtr<UFractureModalTool> ActiveTool;

	// called when PIE is about to start, shuts down active tools
	FDelegateHandle BeginPIEDelegateHandle;
	// calls with the project settings are modified; used to keep the quick settings up to date
	FDelegateHandle ProjectSettingsModifiedHandle;

	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<IDetailsView> FractureSettingsDetailsView;
	TSharedPtr<IStructureDetailsView> OverridesDetailsView;
	TSharedPtr<IDetailsView> HistogramDetailsView;
	TSharedPtr<IDetailsView> OutlinerDetailsView;
	TSharedPtr<SWidget> ToolkitWidget;
	TSharedPtr<SGeometryCollectionOutliner> OutlinerView;
	TSharedPtr<SGeometryCollectionHistogram> HistogramView;
	TWeakPtr<SDockTab> HierarchyTab;
	FMinorTabConfig HierarchyTabInfo;
	TWeakPtr<SDockTab> StatisticsTab;
	FMinorTabConfig StatisticsTabInfo;
	TSharedPtr<SGeometryCollectionStatistics> StatisticsView;
	TArray<TSharedPtr<FString>> AssetLocationModes;
	TSharedPtr<STextComboBox> AssetLocationMode;
	TSharedPtr<SWidget> DataflowProgressBar;

	TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext;
	TSharedPtr<UE::Dataflow::IDataflowEvaluator> DataflowEvaluator;

	// Simple cached statistics to allow use to quickly/heuristically check for stale geometry collection data
	int64 OutlinerCachedBoneCount = 0, OutlinerCachedVertexCount = 0, OutlinerCachedHullCount = 0;

	FDelegateHandle OnFinishedChangingVariableOverridesDelegateHandle;
};

#undef UE_API
