// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"

#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include "FractureModeSettings.generated.h"

#define UE_API FRACTUREEDITOR_API

class FGeometryCollection;

// Control the default asset folder presented when using the "New" tool to create a Geometry Collection in Fracture Mode
UENUM()
enum class EFractureModeNewAssetLocation
{
	/** Default to creating rest collections in the same folder as the source asset. */
	SourceAssetFolder,

	/** Default to creating rest collections in the last folder selected this session. If no folder was selected yet, use the Source Asset Folder. */
	LastUsedFolder,

	/** Default to creating reset collections in the currently-visible Asset Browser folder if available, otherwise use the Last Used Folder. */
	ContentBrowserFolder
};

/**
 * Settings for the Fracture Editor Mode.
 */
UCLASS(MinimalAPI, config = Editor)
class UFractureModeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	// UDeveloperSettings overrides

	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	virtual FName GetSectionName() const override { return FName("FractureMode"); }

	virtual FText GetSectionText() const override { return NSLOCTEXT("FractureModeSettings", "FractureModeSettingsName", "Fracture Mode"); }
	virtual FText GetSectionDescription() const override { return NSLOCTEXT("FractureModeSettings", "FractureModeSettingsDescription", "Configure the Fracture Editor Mode plugin"); }

public:

	/** The default asset folder presented when using the "New" tool to create a Geometry Collection in Fracture Mode */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Asset Location")
	EFractureModeNewAssetLocation NewAssetLocation = EFractureModeNewAssetLocation::SourceAssetFolder;

	/** Default fraction of geometry volume by which a cluster's convex hull volume can exceed the actual geometry volume before instead using the hulls of the children.  0 means the convex volume cannot exceed the geometry volume; 1 means the convex volume is allowed to be 100% larger (2x) the geometry volume. */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Convex Generation Defaults", meta = (ClampMin = "0", DisplayName = "Allow Larger Hull Fraction"))
	float ConvexCanExceedFraction = .5;

	/** Default simplification threshold for convex hulls of new geometry collections */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Convex Generation Defaults", meta = (ClampMin = "0", DisplayName = "Simplification Distance Threshold"))
	float ConvexSimplificationDistanceThreshold = 10;

	/** Default overlap removal setting for convex hulls of new geometry collections */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Convex Generation Defaults|Overlap Removal", meta = (DisplayName = "Remove Overlaps"))
	EConvexOverlapRemoval ConvexRemoveOverlaps = EConvexOverlapRemoval::All;

	/** Default overlap removal shrink percent (in range 0-100) for convex hulls of new geometry collections. Overlap removal will be computed assuming convex shapes will be scaled down by this percentage. */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Convex Generation Defaults|Overlap Removal", meta = (UIMin = "0", ClampMax = "99.9", DisplayName = "Overlap Removal Shrink Percent"))
	float ConvexOverlapRemovalShrinkPercent = 0;

	/** Default fraction of convex hulls for a transform that we can remove before using the hulls of the children */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Convex Generation Defaults|Overlap Removal", meta = (DisplayName = "Max Removal Fraction", ClampMin = ".01", ClampMax = "1"))
	double ConvexFractionAllowRemove = .5;


	/** Default method used to detect proximity of geometry in a new geometry collection */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (DisplayName = "Detection Method"))
	EProximityMethod ProximityMethod = EProximityMethod::Precise;

	/** When Proximity Detection Method is Convex Hull, how close two hulls need to be to be considered as 'in proximity' */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (ClampMin = "0", DisplayName = "Distance Threshold", EditCondition = "ProximityMethod == EProximityMethod::ConvexHull"))
	float ProximityDistanceThreshold = 1;

	/** Whether to automatically transform the proximity graph into a connection graph to be used for simulation */
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (DisplayName = "Use As Connection Graph"))
	bool bProximityUseAsConnectionGraph = false;

	// Method to use to determine the area of the contact for transforms that are connected in the connection graph used for simulation. Only used if "Use As Connection Graph" is enabled.
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (DisplayName = "Connection Contact Method"))
	EConnectionContactMethod ProximityConnectionContactAreaMethod = EConnectionContactMethod::None;

	// Method to use to determine the contact between two pieces, if the Contact Threshold is greater than 0, for the purpose of filtering out too-small contacts
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (DisplayName = "Contact Filter Method"))
	EProximityContactMethod ProximityContactMethod = EProximityContactMethod::MinOverlapInProjectionToMajorAxes;

	// If greater than zero, proximity will be additionally filtered by a 'contact' threshold, in cm, to exclude grazing / corner proximity
	UPROPERTY(config, EditAnywhere, Category = "Fracture Mode|Proximity Detection Defaults", meta = (ClampMin = "0", DisplayName = "Contact Threshold"))
	float ProximityContactThreshold = 0;
	

	// Apply Convex Generation Defaults to a GeometryCollection
	UE_API void ApplyDefaultConvexSettings(FGeometryCollection& GeometryCollection) const;

	// Apply Proximity Detection Defaults to a GeometryCollection
	UE_API void ApplyDefaultProximitySettings(FGeometryCollection& GeometryCollection) const;

	// Apply all default settings to a GeometryCollection
	UE_API void ApplyDefaultSettings(FGeometryCollection& GeometryCollection) const;


	DECLARE_MULTICAST_DELEGATE_TwoParams(UFractureModeSettingsModified, UObject*, FProperty*);
	UFractureModeSettingsModified OnModified;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		OnModified.Broadcast(this, PropertyChangedEvent.Property);

		Super::PostEditChangeProperty(PropertyChangedEvent);
	}


};

#undef UE_API
