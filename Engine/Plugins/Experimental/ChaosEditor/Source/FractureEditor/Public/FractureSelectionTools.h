// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollectionComponent.h"

class UGeometryCollectionComponent;

class FFractureSelectionTools
{
public:
	/**
	 * Sets the selected bones on a geometry collection component
	 *
	 * @param GeometryCollectionComponent	The component with bones being selected
	 * @param BoneIndices					The indices to select
	 * @param bClearCurrentSelection		Whether to clear the current selection set or just append the new indices to it				
	 * @param bAdd							Whether to always add the BoneIndices to the selection, or to toggle them (i.e., de-select if they were previously selected)
	 * @param bSnapToLevel					Whether to snap the selection to the nearest visible bones (in the 3D view and outliner) by searching the parent bones
	 */
	static void ToggleSelectedBones(UGeometryCollectionComponent* GeometryCollectionComponent, const TArray<int32>& BoneIndices, bool bClearCurrentSelection, bool bAdd, bool bSnapToLevel = true);
	static void ClearSelectedBones(UGeometryCollectionComponent* GeometryCollectionComponent);

	static void SelectNeighbors(UGeometryCollectionComponent* GeometryCollectionComponent);
};