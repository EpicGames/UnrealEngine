// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "FractureSettings.generated.h"

class UGeometryCollection;

UENUM()
enum class EFractureSelectionDisplayMode : uint8
{
	/** Use a glow material to highlight the selection - Nanite Geometry Collection will ignore it and use bounding boxes instead */
	Highlight,

	/** Use bounding boxes to show selection - Nanite Geometry Collection will default to that */
	BoundingBox,

	/** do not display bone selection at all  */
	None,
};


/** Settings specifically related to viewing fractured meshes **/
UCLASS()
class UFractureSettings: public UObject
{

	GENERATED_BODY()
public:
	UFractureSettings(const FObjectInitializer& ObjInit);

	/** Amount to expand the displayed Geometry Collection bones into an 'exploded view' */
	UPROPERTY(EditAnywhere, Category = ViewSettings, meta = (DisplayName = "Explode Amount", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float ExplodeAmount;

	/** Current level of the Geometry Collection displayed */
	UPROPERTY(EditAnywhere, Category = ViewSettings, meta = (DisplayName = "Fracture Level", UIMin = "-1" ))
	int32 FractureLevel;

	/** When active, only show selected bones */
	UPROPERTY(EditAnywhere, Category = ViewSettings, meta = (DisplayName = "Hide Unselected"))
	bool bHideUnselected;

	/** How to display the selection in the viewport */
	UPROPERTY(EditAnywhere, Category = ViewSettings, meta = (DisplayName = "Highlight Selected"))
	EFractureSelectionDisplayMode SelectionDisplayMode;

	UPROPERTY(VisibleAnywhere, Category = ViewSettings)
	TWeakObjectPtr<const UGeometryCollection> RestCollection = nullptr;

};