// Copyright Epic Games, Inc. All Rights Reserved.

#include "FractureTool.h"

#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "EditorModeManager.h"
#include "FractureEditorMode.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"

#include "FractureToolContext.h"
#include "FractureSelectionTools.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FractureTool)


DEFINE_LOG_CATEGORY(LogFractureTool);

#define LOCTEXT_NAMESPACE "FractureTool"

void UFractureToolSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (OwnerTool != nullptr)
	{
		OwnerTool->PostEditChangeProperty(PropertyChangedEvent);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UFractureToolSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (OwnerTool != nullptr)
	{
		OwnerTool->PostEditChangeChainProperty(PropertyChangedEvent);
	}
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}




const TSharedPtr<FUICommandInfo>& UFractureActionTool::GetUICommandInfo() const
{
	return UICommandInfo;
}

UFractureActionTool::FModifyContextScope::FModifyContextScope(UFractureActionTool* ActionTool, FFractureToolContext* FractureContext, bool bWantPhysicsUpdate) : ActionTool(ActionTool), FractureContext(FractureContext)
{
	check(FractureContext);
	check(ActionTool);
	FractureContext->GetFracturedGeometryCollection()->Modify();
	FractureContext->GetGeometryCollectionComponent()->Modify();
	bool bHasPhysicsState = FractureContext->GetGeometryCollectionComponent()->HasValidPhysicsState();
	bNeedPhysicsUpdate = bWantPhysicsUpdate && bHasPhysicsState;
	if (bNeedPhysicsUpdate)
	{
		FractureContext->GetGeometryCollectionComponent()->DestroyPhysicsState();
	}
}

UFractureActionTool::FModifyContextScope::~FModifyContextScope()
{
	if (bNeedPhysicsUpdate)
	{
		FractureContext->GetGeometryCollectionComponent()->RecreatePhysicsState();
	}

	UFractureEditorMode* FractureMode = Cast<UFractureEditorMode>(GLevelEditorModeTools().GetActiveScriptableMode(UFractureEditorMode::EM_FractureEditorModeId));
	if (!FractureMode)
	{
		return;
	}
	TSharedPtr<FModeToolkit> ModeToolkit = FractureMode->GetToolkit().Pin();
	if (ModeToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = (FFractureEditorModeToolkit*)ModeToolkit.Get();
		ActionTool->Refresh(*FractureContext, Toolkit);
	}
}


bool UFractureActionTool::CanExecute() const
{
	return IsGeometryCollectionSelected();
}

bool UFractureActionTool::IsGeometryCollectionSelected()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<ULevel*> UniqueLevels;
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
			Actor->GetComponents(GeometryCollectionComponents, true);

			if (GeometryCollectionComponents.Num() > 0)
			{
				return true;
			}
		}
	}
	return false;
}

bool UFractureActionTool::IsStaticMeshSelected()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents, true);

			if (StaticMeshComponents.Num() > 0)
			{
				return true;
			}
		}
	}
	return false;
}

void UFractureActionTool::AddSingleRootNodeIfRequired(UGeometryCollection* GeometryCollectionObject)
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
	if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
	{
		if (FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(GeometryCollection))
		{
			FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(GeometryCollection);
		}
	}
}

void UFractureActionTool::AddAdditionalAttributesIfRequired(UGeometryCollection* GeometryCollectionObject)
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
	if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
	{
		if (!GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
		{
			FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
		}
	}
}

void UFractureActionTool::GetSelectedGeometryCollectionComponents(TSet<UGeometryCollectionComponent*>& GeomCompSelection, bool bFilterForUniqueRestCollections)
{
	USelection* SelectionSet = GEditor->GetSelectedActors();
	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectionSet->Num());
	SelectionSet->GetSelectedObjects(SelectedActors);

	GeomCompSelection.Empty(SelectionSet->Num());

	TSet<const UGeometryCollection*> AddedRestCollections;

	for (AActor* Actor : SelectedActors)
	{
		TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
		Actor->GetComponents(GeometryCollectionComponents);
		if (bFilterForUniqueRestCollections)
		{
			for (UGeometryCollectionComponent* Component : GeometryCollectionComponents)
			{
				bool bWasInSet = false;
				AddedRestCollections.FindOrAdd(Component->GetRestCollection(), &bWasInSet);
				if (!bWasInSet)
				{
					GeomCompSelection.Add(Component);
				}
			}
		}
		else
		{
			GeomCompSelection.Append(GeometryCollectionComponents);
		}
	}
}

TArray<FString> UFractureActionTool::GetSelectedComponentMaterialNames(bool bIncludeDefault, bool bUseFullNamesIfPossible)
{
	TArray<FString> MaterialNames;

	if (bIncludeDefault)
	{
		MaterialNames.Add(LOCTEXT("AutomaticMaterialOption", "Automatic").ToString());
	}
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	if (GeomCompSelection.Num() > 1 || !bUseFullNamesIfPossible)
	{
		int32 MinMaterials = -1;
		for (UGeometryCollectionComponent* Component : GeomCompSelection)
		{
			int32 NumMaterials = Component->GetNumMaterials();
			if (MinMaterials < 0 || NumMaterials < MinMaterials)
			{
				MinMaterials = NumMaterials;
			}
		}
		for (int32 MatIdx = 0; MatIdx < MinMaterials; ++MatIdx)
		{
			MaterialNames.Add(FString::Printf(TEXT("[%d]"), MatIdx));
		}
	}
	else if (GeomCompSelection.Num() == 1)
	{
		for (UGeometryCollectionComponent* Component : GeomCompSelection)
		{
			int32 NumMaterials = Component->GetNumMaterials();
			for (int32 MatIdx = 0; MatIdx < NumMaterials; ++MatIdx)
			{
				UMaterialInterface* Material = Component->GetMaterial(MatIdx);
				FString MaterialName = Material ? Material->GetName() : LOCTEXT("NoMaterialName", "None").ToString();
				MaterialNames.Add(FString::Printf(TEXT("[%d] %s"), MatIdx, *MaterialName));
			}
		}
	}

	return MaterialNames;
}

void UFractureActionTool::Refresh(UGeometryCollectionComponent* GeometryCollectionComponent, FFractureEditorModeToolkit* Toolkit, const TArray<int32>& SetSelection, bool bClearSelection, bool bMustUpdateBoneColors)
{
	FGeometryCollectionEdit CollectionEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None);
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = CollectionEdit.GetRestCollection()->GetGeometryCollection();

	FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(GeometryCollectionPtr.Get(), -1);

	Toolkit->RegenerateOutliner();
	Toolkit->RegenerateHistogram();

	if (bMustUpdateBoneColors)
	{
		GeometryCollectionComponent->EditBoneSelection(true);
	}

	if (bClearSelection)
	{
		FFractureSelectionTools::ClearSelectedBones(GeometryCollectionComponent);
	}
	else
	{
		FFractureSelectionTools::ToggleSelectedBones(GeometryCollectionComponent, SetSelection, true, true);
	}

	CollectionEdit.GetRestCollection()->RebuildRenderData();
}

void UFractureActionTool::Refresh(FFractureToolContext& Context, FFractureEditorModeToolkit* Toolkit, bool bClearSelection)
{
	UGeometryCollectionComponent* GeometryCollectionComponent = Context.GetGeometryCollectionComponent();
	Refresh(GeometryCollectionComponent, Toolkit, Context.GetSelection(), bClearSelection);
}

void UFractureActionTool::SetOutlinerComponents(TArray<FFractureToolContext>& InContexts, FFractureEditorModeToolkit* Toolkit)
{
	// Extract components from contexts
	TArray<UGeometryCollectionComponent*> Components;
	for (FFractureToolContext& Context : InContexts)
	{
		if (Context.GetGeometryCollection())
		{
			Components.AddUnique(Context.GetGeometryCollectionComponent());
		}
	}
	Toolkit->SetOutlinerComponents(Components);
}

void UFractureActionTool::ClearProximity(FGeometryCollection* GeometryCollection)
{
	if (GeometryCollection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		GeometryCollection->RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
	}
}

TArray<FFractureToolContext> UFractureActionTool::GetFractureToolContexts() const
{
	TArray<FFractureToolContext> Contexts;

	// A context is gathered for each selected GeometryCollection component.
	TSet<UGeometryCollectionComponent*> GeomCompSelection;
	GetSelectedGeometryCollectionComponents(GeomCompSelection);
	for (UGeometryCollectionComponent* GeometryCollectionComponent : GeomCompSelection)
	{
		Contexts.Emplace(GeometryCollectionComponent);
	}

	return Contexts;
}


void UFractureModalTool::EnumerateVisualizationMapping(const FVisualizationMappings& Mappings, int32 ArrayNum, TFunctionRef<void(int32 Idx, FVector ExplodedVector)> Func) const
{
	for (int32 MappingIdx = 0; MappingIdx < Mappings.Mappings.Num(); MappingIdx++)
	{
		const FVisualizationMappings::FIndexMapping& Mapping = Mappings.Mappings[MappingIdx];
		FVector Offset = Mappings.GetExplodedVector(MappingIdx, VisualizedCollections[Mapping.CollectionIdx]);
		int32 EndIdx = Mappings.GetEndIdx(MappingIdx, ArrayNum);
		for (int32 Idx = Mapping.StartIdx; Idx < EndIdx; Idx++)
		{
			Func(Idx, Offset);
		}
	}
}


void UFractureModalTool::OverrideEditorViewFlagsForLineRendering()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				Viewport.EnableOverrideEngineShowFlags([](FEngineShowFlags& Flags)
					{
						Flags.SetTemporalAA(false);
						Flags.SetMotionBlur(false);
					});
			}
		}
	}
}

void UFractureModalTool::RestoreEditorViewFlags()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				Viewport.DisableOverrideEngineShowFlags();
			}
		}
	}
}

void UFractureModalTool::NotifyOfPropertyChangeByTool(UFractureToolSettings* PropertySet) const
{
	OnPropertyModifiedDirectlyByTool.Broadcast(PropertySet);
}


void UFractureModalTool::Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
{
	TSharedPtr<FFractureEditorModeToolkit> ModeToolkit = InToolkit.Pin();
	if (ModeToolkit.IsValid())
	{
		FFractureEditorModeToolkit* Toolkit = ModeToolkit.Get();

		TArray<FFractureToolContext> FractureContexts = GetFractureToolContexts();

		for (FFractureToolContext& FractureContext : FractureContexts)
		{
			if (FractureContext.GetGeometryCollection())
			{
				FGeometryCollectionEdit EditCollection(FractureContext.GetGeometryCollectionComponent(), GeometryCollection::EEditUpdate::RestPhysicsDynamic, !ExecuteUpdatesShape());

				int32 InitialNumTransforms = FractureContext.GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup);
				int32 FirstNewGeometryIndex = ExecuteFracture(FractureContext);

				if (FirstNewGeometryIndex > INDEX_NONE)
				{
					PostFractureProcess(FractureContext, FirstNewGeometryIndex);

					FractureContext.GenerateGuids(FirstNewGeometryIndex);

					// Based on the first new geometry index, generate a list of new transforms generated by the fracture.
					const TManagedArray<int32>& TransformIndex = FractureContext.GetGeometryCollection()->GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
					int32 LastGeometryIndex = TransformIndex.Num();

					TArray<int32> NewTransforms;
					NewTransforms.Reserve(LastGeometryIndex - FirstNewGeometryIndex);
					for (int32 GeometryIndex = FirstNewGeometryIndex; GeometryIndex < LastGeometryIndex; ++GeometryIndex)
					{
						NewTransforms.Add(TransformIndex[GeometryIndex]);
					}

					FractureContext.SetSelection(NewTransforms);
				}
				else
				{
					// either no update was done, or the updated range was not expressible as a final geometry index
					// -- in the latter case, selection may become invalid, so we should clear it
					int32 NumTransforms = FractureContext.GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup);
					TArray<int32>& Selection = FractureContext.GetSelection();
					if (InitialNumTransforms != NumTransforms)
					{
						Selection.Empty();
					}
					FractureContext.Sanitize(false);
				}

				Refresh(FractureContext, Toolkit);
			}
		}

		SetOutlinerComponents(FractureContexts, Toolkit);
	}
}

bool UFractureModalTool::CanExecute() const
{
	if (!IsGeometryCollectionSelected())
	{
		return false;
	}
	/*
	if (IsStaticMeshSelected())
	{
		return false;
	}
	*/
	return true;
}

void UFractureModalTool::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	FractureContextChanged();
}


void UFractureModalTool::OnComponentTransformChangedInternal(USceneComponent* InRootComponent, ETeleportType Teleport)
{
	if (UGeometryCollectionComponent* GeomColl = Cast<UGeometryCollectionComponent>(InRootComponent))
	{
		bool bIsSel = GeomColl->IsSelected();
		bool bActorIsSel = GeomColl->GetOwner()->IsSelected();
		if (bIsSel || bActorIsSel)
		{
			OnComponentTransformChanged(GeomColl);
		}
	}
}


FVector FVisualizationMappings::GetExplodedVector(int32 MappingIdx, const UGeometryCollectionComponent* CollectionComponent) const
{
	int32 BoneIdx = Mappings[MappingIdx].BoneIdx;
	if (BoneIdx != INDEX_NONE && CollectionComponent)
	{
		const FGeometryCollection& Collection = *CollectionComponent->GetRestCollection()->GetGeometryCollection();
		if (Collection.HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup))
		{
			FVector Offset = (FVector)Collection.GetAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup)[BoneIdx];
			return CollectionComponent->GetOwner()->GetActorTransform().TransformVector(Offset);
		}
	}
	return FVector::ZeroVector;
}

#undef LOCTEXT_NAMESPACE
