// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Textures/SlateIcon.h"
#include "SceneManagement.h"

#include "Framework/Commands/UICommandInfo.h"
#include "FractureEditorCommands.h"
#include "FractureEditorModeToolkit.h"

#include "Tools/EdModeInteractiveToolsContext.h"

#include "FractureTool.generated.h"

class UGeometryCollection;
class UFractureModalTool;
class FFractureToolContext;

template <typename T>
class TManagedArray;

DECLARE_LOG_CATEGORY_EXTERN(LogFractureTool, Log, All);

UCLASS(Abstract, config = EditorPerProjectUserSettings)
class UFractureToolSettings : public UObject
{
	GENERATED_BODY()
public:
	UFractureToolSettings(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	UPROPERTY()
	TObjectPtr<UFractureModalTool> OwnerTool;
};

class FFractureToolContext;

/** Tools derived from this class should require parameter inputs from the user, only the bone selection. */
UCLASS(Abstract)
class UFractureActionTool : public UObject
{
public:
	GENERATED_BODY()

	UFractureActionTool(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	/** This is the Text that will appear on the tool button to execute the tool **/
	virtual FText GetDisplayText() const { return FText(); }
	virtual FText GetTooltipText() const { return FText(); }

	virtual FSlateIcon GetToolIcon() const { return FSlateIcon(); }

	/** Executes the command.  Derived types need to be implemented in a thread safe way*/
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) {}
	virtual bool CanExecute() const;

	/** Gets the UI command info for this command */
	const TSharedPtr<FUICommandInfo>& GetUICommandInfo() const;

	virtual void RegisterUICommand(FFractureEditorCommands* BindingContext) {}

	virtual TArray<FFractureToolContext> GetFractureToolContexts() const;

	// Scope sets up modification and updates UI after
	struct FModifyContextScope
	{
		UFractureActionTool* ActionTool;
		FFractureToolContext* FractureContext;
		bool bNeedPhysicsUpdate;

		FModifyContextScope(UFractureActionTool* ActionTool, FFractureToolContext* FractureContext, bool bWantPhysicsUpdate = true);

		~FModifyContextScope();
	};

public:

	/**
	 * Get the Geometry Collection components in the current selection
	 * @param GeomCompSelection					the set to fill with Geometry Collection components
	 * @param bFilterForUniqueRestCollections	if true, and selection contains multiple components backed by the same Rest Collection asset, the selection will be filtered to only contain one component per asset
	 */
	static void GetSelectedGeometryCollectionComponents(TSet<UGeometryCollectionComponent*>& GeomCompSelection, bool bFilterForUniqueRestCollections = false);
	static TArray<FString> GetSelectedComponentMaterialNames(bool bIncludeDefault, bool bUseFullNamesIfPossible = true);

	
protected:
	static bool IsStaticMeshSelected();
	static bool IsGeometryCollectionSelected();
	static void AddSingleRootNodeIfRequired(UGeometryCollection* GeometryCollectionObject);
	static void AddAdditionalAttributesIfRequired(UGeometryCollection* GeometryCollectionObject);
	static void Refresh(FFractureToolContext& Context, FFractureEditorModeToolkit* Toolkit, bool bClearSelection=false);
	static void Refresh(UGeometryCollectionComponent* Component, FFractureEditorModeToolkit* Toolkit, const TArray<int32>& SetSelection, bool bClearSelection=false, bool bMustUpdateBoneColors=false);
	static void SetOutlinerComponents(TArray<FFractureToolContext>& InContexts, FFractureEditorModeToolkit* Toolkit);
	static void ClearProximity(FGeometryCollection* GeometryCollection);


protected:
	TSharedPtr<FUICommandInfo> UICommandInfo;

};

// Helper structure to correspond a UFractureModalTool's VisualizedCollections with elements of the tool's visualizations
struct FVisualizationMappings
{
	// helper structure to map visualized geometry to source geometry and the bone that it corresponds to (if applicable)
	struct FIndexMapping
	{
		int32 CollectionIdx = INDEX_NONE; // index into VisualizedCollections
		int32 BoneIdx = INDEX_NONE; // transform (bone) index in geometry collection that corresponds to the visualization
		int32 StartIdx = INDEX_NONE; // index of first element of visualization-related array that uses this map (e.g. index into VoronoiSites)
	};

	TArray<FIndexMapping> Mappings;

	FVector GetExplodedVector(int32 MappingIdx, const UGeometryCollectionComponent* Collection) const;

	void Empty()
	{
		Mappings.Empty();
	}

	void AddMapping(int32 CollectionIdx, int32 BoneIdx, int32 StartIdx = INDEX_NONE)
	{
		Mappings.Add({ CollectionIdx, BoneIdx, StartIdx });
	}
	int32 GetEndIdx(int32 CorrespondIdx, int32 ArrayNum) const
	{
		if (CorrespondIdx + 1 < Mappings.Num())
		{
			return Mappings[CorrespondIdx + 1].StartIdx;
		}
		else
		{
			return ArrayNum;
		}
	}
};

/** Tools derived from this class provide parameter details and operate modally. */
UCLASS(Abstract)
class UFractureModalTool : public UFractureActionTool
{
public:
	GENERATED_BODY()

	UFractureModalTool(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	virtual TArray<UObject*> GetSettingsObjects() const { return TArray<UObject*>(); }
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) {}

	/** This is the Text that will appear on the button to execute the fracture **/
	virtual FText GetApplyText() const { return FText(); }

	/** Executes the command.  Derived types need to be implemented in a thread safe way*/
	virtual void Execute(TWeakPtr<FFractureEditorModeToolkit> InToolkit) override;
	virtual bool CanExecute() const override;
	// @return true if the edit will change the shape data of the geometry collection(s) (including changing the clusters) -- used to indicate whether we need to update the convex hulls
	virtual bool ExecuteUpdatesShape() const
	{
		return true;
	}

	/** Executes function that generates new geometry. Returns the first new geometry index. */
	virtual int32 ExecuteFracture(const FFractureToolContext& FractureContext) { return INDEX_NONE; }

	// Optional processing after a successful fracture (i.e., will not be called if ExecuteFracture returns INDEX_NONE)
	virtual void PostFractureProcess(const FFractureToolContext& FractureContext, int32 FirstNewGeometryIndex)
	{

	}

	virtual void OnTick(float DeltaTime) {}

	/** Draw callback from edmode*/
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) {}
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) {}

	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void FractureContextChanged() {}
	
	// Called when the selection changes, and when the modal tool is entered
	virtual void SelectedBonesChanged() {}

	// Called when the modal tool is entered
	virtual void Setup(TWeakPtr<FFractureEditorModeToolkit> InToolkit)
	{
		GEngine->OnComponentTransformChanged().AddUObject(this, &UFractureModalTool::OnComponentTransformChangedInternal);
	}

	// Called when the modal tool is exited (on switching to a new modal tool or exiting the fracture editor mode)
	virtual void Shutdown() 
	{
		GEngine->OnComponentTransformChanged().RemoveAll(this);
		ClearVisualizations();
		RestoreEditorViewFlags();
	}

	// Called when a selected geometry collection component is moved in the scene
	virtual void OnComponentTransformChanged(UGeometryCollectionComponent* Component)
	{
		FractureContextChanged();
	}

	/**
	 * Call after changing properties internally in the tool to allow external views of the property
	 * to update properly. This is meant as an outward notification mechanism, not a way to to
	 * pass along notifications, so don't call this if the property is changed externally (i.e., this
	 * should not usually be called from OnPropertyModified unless the tool adds changes of its own).
	 */
	virtual void NotifyOfPropertyChangeByTool(UFractureToolSettings* PropertySet) const;

	/**
	 * OnPropertyModifiedDirectlyByTool is broadcast when a property is changed internally by the tool.
	 * This allows any external display of such properties to update. In a DetailsView, for instance,
	 * it refreshes certain cached states such as edit condition states for other properties.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnToolPropertyInternallyModified, UObject*);
	OnToolPropertyInternallyModified OnPropertyModifiedDirectlyByTool;

protected:

	// Geometry collection components referenced by visualizations
	UPROPERTY()
	TArray<TObjectPtr<UGeometryCollectionComponent>> VisualizedCollections;

	virtual void ClearVisualizations()
	{
		VisualizedCollections.Empty();
	}

	void EnumerateVisualizationMapping(const FVisualizationMappings& Mappings, int32 ArrayNum, TFunctionRef<void(int32 Idx, FVector ExplodedVector)> Func) const;
	
	void OnComponentTransformChangedInternal(USceneComponent* InRootComponent, ETeleportType Teleport);

	// Call to override editor view flags to disable temporal AA and motion blur, which make the lines look bad
	void OverrideEditorViewFlagsForLineRendering();
	// Restore editor view flags
	void RestoreEditorViewFlags();

};


/** Tools derived from this class provide parameter details, operate modally and use a viewport manipulator to set certain parameters. */
UCLASS(Abstract)
class UFractureInteractiveTool : public UFractureModalTool
{
public:
	GENERATED_BODY()

	UFractureInteractiveTool(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	// #todo (bmiller) implement interactive widgets

};

